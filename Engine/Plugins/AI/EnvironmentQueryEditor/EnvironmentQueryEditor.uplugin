{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Environment Query Editor",
	"Description" : "Allows editing of Environment Query assets, which are used by the AI to collect data about the environment/world",
	"Category" : "AI",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"Installed" : false,
	"Modules" : [
		{
			"Name" : "EnvironmentQueryEditor",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PreDefault"
		}
	],
	"Plugins" : [
		{
			"Name" : "AISupport",
			"Enabled" : true
		}
	]
}