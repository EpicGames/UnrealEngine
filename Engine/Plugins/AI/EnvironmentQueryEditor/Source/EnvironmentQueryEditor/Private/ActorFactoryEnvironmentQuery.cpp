// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactoryEnvironmentQuery.h"

#include "AssetRegistry/AssetData.h"
#include "EnvironmentQuery/EnvQuery.h"
#include "EnvironmentQuery/EQSTestingPawn.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryEnvironmentQuery)

#define LOCTEXT_NAMESPACE "ActorFactoryEnvironmentQuery"

UActorFactoryEnvironmentQuery::UActorFactoryEnvironmentQuery()
{
	DisplayName = LOCTEXT("EnvironmentQueryActorDisplayName", "Environment Query");
	NewActorClass = AEQSTestingPawn::StaticClass();
}

void UActorFactoryEnvironmentQuery::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UEnvQuery* EnvQuery = CastChecked<UEnvQuery>(Asset);
	AEQSTestingPawn* NewEQSTestingPawn = CastChecked<AEQSTestingPawn>(NewActor);

	NewEQSTestingPawn->QueryTemplate = EnvQuery;
	NewEQSTestingPawn->QueryTemplate->CollectQueryParams(*NewEQSTestingPawn, NewEQSTestingPawn->QueryConfig);
	NewEQSTestingPawn->RunEQSQuery();
}

bool UActorFactoryEnvironmentQuery::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.GetClass()->IsChildOf(UEnvQuery::StaticClass()))
	{
		OutErrorMsg = LOCTEXT("NoEnvironmentQuery", "A valid environment query must be specified.");
		return false;
	}

	// Not calling Super since it's checking if AssetData matches the actor class we want to spawn, and that's not the case here.
	// We're spawning a EQSTestingPawn to put a given EQS query in the world context.
	return true;
}

UObject* UActorFactoryEnvironmentQuery::GetAssetFromActorInstance(AActor* ActorInstance)
{
	return CastChecked<AEQSTestingPawn>(ActorInstance)->QueryTemplate;
}

#undef LOCTEXT_NAMESPACE
