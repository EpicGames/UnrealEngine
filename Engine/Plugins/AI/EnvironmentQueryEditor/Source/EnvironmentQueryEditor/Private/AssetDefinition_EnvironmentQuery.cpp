// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_EnvironmentQuery.h"
#include "EnvironmentQuery/EnvQuery.h"
#include "EnvironmentQueryEditorModule.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_EnvironmentQuery)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText UAssetDefinition_EnvironmentQuery::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_EnvironmentQuery", "Environment Query");
}

FLinearColor UAssetDefinition_EnvironmentQuery::GetAssetColor() const
{
	return FColor(201, 29, 85);
}

TSoftClassPtr<> UAssetDefinition_EnvironmentQuery::GetAssetClass() const
{
	return UEnvQuery::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_EnvironmentQuery::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::AI, LOCTEXT("AssetDefinition_EnvironmentQuerySubMenu", "NPC"), ECategoryMenuType::Section) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_EnvironmentQuery::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FEnvironmentQueryEditorModule& EnvironmentQueryEditorModule = FModuleManager::LoadModuleChecked<FEnvironmentQueryEditorModule>("EnvironmentQueryEditor");
	for (UEnvQuery* Query : OpenArgs.LoadObjects<UEnvQuery>())
	{
		EnvironmentQueryEditorModule.CreateEnvironmentQueryEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, Query);
	}
	return EAssetCommandResult::Handled;
}
#undef LOCTEXT_NAMESPACE
