// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations/EnvDirectionCustomization.h"
#include "Widgets/Text/STextBlock.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "FEnvQueryCustomization"

TSharedRef<IPropertyTypeCustomization> FEnvDirectionCustomization::MakeInstance( )
{
	return MakeShareable(new FEnvDirectionCustomization);
}

void FEnvDirectionCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	// create struct header
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Text(this, &FEnvDirectionCustomization::GetShortDescription)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];

	ModeProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvDirection,DirMode));
	if (ModeProp.IsValid())
	{
		FSimpleDelegate OnModeChangedDelegate = FSimpleDelegate::CreateSP(this, &FEnvDirectionCustomization::OnModeChanged);
		ModeProp->SetOnPropertyValueChanged(OnModeChangedDelegate);
	}
	
	OnModeChanged();
}

void FEnvDirectionCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	StructBuilder.AddProperty(ModeProp.ToSharedRef());

	TSharedPtr<IPropertyHandle> PropFrom = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvDirection,LineFrom));
	StructBuilder.AddProperty(PropFrom.ToSharedRef())
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvDirectionCustomization::GetTwoPointsVisibility)));

	TSharedPtr<IPropertyHandle> PropTo = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvDirection,LineTo));
	StructBuilder.AddProperty(PropTo.ToSharedRef())
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvDirectionCustomization::GetTwoPointsVisibility)));

	TSharedPtr<IPropertyHandle> PropRot = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvDirection,Rotation));
	StructBuilder.AddProperty(PropRot.ToSharedRef())
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvDirectionCustomization::GetRotationVisibility)));
}

FText FEnvDirectionCustomization::GetShortDescription() const
{
	return bIsRotation ? LOCTEXT("DirectionShortDescRotatation", "context's rotation...") : LOCTEXT("DirectionShortDescBetweenTwoPoints", "between two contexts...");
}

EVisibility FEnvDirectionCustomization::GetTwoPointsVisibility() const
{
	return bIsRotation ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility FEnvDirectionCustomization::GetRotationVisibility() const
{
	return bIsRotation ? EVisibility::Visible : EVisibility::Collapsed;
}

void FEnvDirectionCustomization::OnModeChanged()
{
	uint8 EnumValue = 0;
	ModeProp->GetValue(EnumValue);

	bIsRotation = (EnumValue == EEnvDirection::Rotation);
}

#undef LOCTEXT_NAMESPACE
