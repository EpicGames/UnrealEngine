// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

struct EVisibility;

class IPropertyHandle;

class FEnvDirectionCustomization : public IPropertyTypeCustomization
{
public:

	//~ Begin IPropertyTypeCustomization Interface
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	//~ End IPropertyTypeCustomization Interface

	static TSharedRef<IPropertyTypeCustomization> MakeInstance( );

protected:

	TSharedPtr<IPropertyHandle> ModeProp;
	bool bIsRotation;

	FText GetShortDescription() const;
	EVisibility GetTwoPointsVisibility() const;
	EVisibility GetRotationVisibility() const;
	void OnModeChanged();
};
