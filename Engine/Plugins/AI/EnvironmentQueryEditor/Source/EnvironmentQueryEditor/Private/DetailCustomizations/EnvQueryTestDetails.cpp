// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations/EnvQueryTestDetails.h"
#include "SlateOptMacros.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "STestFunctionWidget.h"

#define LOCTEXT_NAMESPACE "EnvQueryTestDetails"

TSharedRef<IDetailCustomization> FEnvQueryTestDetails::MakeInstance()
{
	return MakeShareable( new FEnvQueryTestDetails );
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FEnvQueryTestDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	AllowWriting = false;

	TArray<TWeakObjectPtr<UObject> > EditedObjects;
	DetailLayout.GetObjectsBeingCustomized(EditedObjects);
	for (int32 i = 0; i < EditedObjects.Num(); i++)
	{
		const UEnvQueryTest* EditedTest = Cast<const UEnvQueryTest>(EditedObjects[i].Get());
		if (EditedTest)
		{
			MyTest = MakeWeakObjectPtr(const_cast<UEnvQueryTest*>(EditedTest));
			break;
		}
	}

	// Initialize all handles
	FilterTypeHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, FilterType));
	ScoreEquationHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ScoringEquation));
	TestPurposeHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, TestPurpose));
	ScoreHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ScoringFactor));

	ScoreNormalizationTypeHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, NormalizationType));
	ScoreReferenceValueHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ReferenceValue));
	MultipleContextScoreOpHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, MultipleContextScoreOp));
	MultipleContextFilterOpHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, MultipleContextFilterOp));

	ClampMinTypeHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ClampMinType));
	ClampMaxTypeHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ClampMaxType));

	ScoreClampMinHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ScoreClampMin));
	FloatValueMinHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, FloatValueMin));

	ScoreClampMaxHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ScoreClampMax));
	FloatValueMaxHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, FloatValueMax));

	// Build combo box data values
	BuildFilterTestValues();
	BuildScoreEquationValues();
	BuildScoreClampingTypeValues(true, ClampMinTypeValues);
	BuildScoreClampingTypeValues(false, ClampMaxTypeValues);
	
	IDetailCategoryBuilder& TestCategory = DetailLayout.EditCategory("Test");
	IDetailPropertyRow& TestPurposeRow = TestCategory.AddProperty(TestPurposeHandle);

	IDetailCategoryBuilder& FilterCategory = DetailLayout.EditCategory("Filter");

	IDetailPropertyRow& FilterTypeRow = FilterCategory.AddProperty(FilterTypeHandle);
	FilterTypeRow.CustomWidget()
		.NameContent()
		[
			FilterTypeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FEnvQueryTestDetails::OnGetFilterTestContent)
			.ContentPadding(FMargin( 2.0f, 2.0f ))
			.ButtonContent()
			[
				SNew(STextBlock) 
				.Text(this, &FEnvQueryTestDetails::GetCurrentFilterTestDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
  	FilterTypeRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetFloatFilterVisibility)));

	// filters
	IDetailPropertyRow& FloatValueMinRow = FilterCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, FloatValueMin)));
	FloatValueMinRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetVisibilityOfFloatValueMin)));

	IDetailPropertyRow& FloatValueMaxRow = FilterCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, FloatValueMax)));
	FloatValueMaxRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetVisibilityOfFloatValueMax)));

	IDetailPropertyRow& BoolValueRow = FilterCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, BoolValue)));
	BoolValueRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetBoolValueVisibility)));

	IDetailPropertyRow& MultipleContextFilterOpRow = FilterCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, MultipleContextFilterOp)));
	MultipleContextFilterOpRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetVisibilityForFiltering)));

	// required when it's created for "scoring only" tests
	IDetailGroup& HackToEnsureFilterCategoryIsVisible = FilterCategory.AddGroup("HackForVisibility", FText::GetEmpty());
	HackToEnsureFilterCategoryIsVisible.HeaderRow().Visibility(EVisibility::Hidden);

	// Scoring
	IDetailCategoryBuilder& ScoreCategory = DetailLayout.EditCategory("Score");

	//----------------------------
	// BEGIN Scoring: Clamping
	IDetailGroup& ClampingGroup = ScoreCategory.AddGroup("Clamping", LOCTEXT("ClampingLabel", "Clamping"));
	ClampingGroup.HeaderRow()
	[
		SNew(STextBlock)
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetClampingVisibility)))
		.Text(FText::FromString("Clamping"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
	
	// Drop-downs for setting type of lower and upper bound normalization
	IDetailPropertyRow& ClampMinTypeRow = ClampingGroup.AddPropertyRow(ClampMinTypeHandle.ToSharedRef());
	ClampMinTypeRow.CustomWidget()
		.NameContent()
		[
			ClampMinTypeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FEnvQueryTestDetails::OnGetClampMinTypeContent)
			.ContentPadding(FMargin( 2.0f, 2.0f ))
			.ButtonContent()
			[
				SNew(STextBlock) 
				.Text(this, &FEnvQueryTestDetails::GetClampMinTypeDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	ClampMinTypeRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetFloatScoreVisibility)));

	// Lower Bound for normalization of score if specified independently of filtering.
	IDetailPropertyRow& ScoreClampMinRow = ClampingGroup.AddPropertyRow(ScoreClampMinHandle.ToSharedRef());
	ScoreClampMinRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetVisibilityOfScoreClampMinimum)));

	// Lower Bound for scoring when tied to filter minimum.
	if (FloatValueMinHandle->IsValidHandle())
	{
		IDetailPropertyRow& FloatValueMinForClampingRow = ClampingGroup.AddPropertyRow(FloatValueMinHandle.ToSharedRef());
		FloatValueMinForClampingRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetVisibilityOfValueMinForScoreClamping)));
		FloatValueMinForClampingRow.ToolTip(LOCTEXT("FloatFilterMinForClampingRowToolTip", "See Filter Thresholds under the Filter tab.  Values lower than this (before clamping) cause the item to be thrown out as invalid.  Values are normalized with this value as the minimum, so items with this value will have a normalized score of 0."));
		FloatValueMinForClampingRow.EditCondition(TAttribute<bool>(this, &FEnvQueryTestDetails::AllowWritingToFiltersFromScore), NULL);
	}

	if (ClampMaxTypeHandle->IsValidHandle())
	{
		IDetailPropertyRow& ClampMaxTypeRow = ClampingGroup.AddPropertyRow(ClampMaxTypeHandle.ToSharedRef());
		ClampMaxTypeRow.CustomWidget()
			.NameContent()
			[
				ClampMaxTypeHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FEnvQueryTestDetails::OnGetClampMaxTypeContent)
				.ContentPadding(FMargin( 2.0f, 2.0f ))
				.ButtonContent()
				[
					SNew(STextBlock) 
					.Text(this, &FEnvQueryTestDetails::GetClampMaxTypeDesc)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			];
		ClampMaxTypeRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetFloatScoreVisibility)));
	}

	// Upper Bound for normalization of score if specified independently of filtering.
	if (ScoreClampMaxHandle->IsValidHandle())
	{
		IDetailPropertyRow& ScoreClampMaxRow = ClampingGroup.AddPropertyRow(ScoreClampMaxHandle.ToSharedRef());
		ScoreClampMaxRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetVisibilityOfScoreClampMaximum)));
	}
	
	if (FloatValueMaxHandle->IsValidHandle())
	{
		// Upper Bound for scoring when tied to filter maximum.
		IDetailPropertyRow& FloatValueMaxForClampingRow = ClampingGroup.AddPropertyRow(FloatValueMaxHandle.ToSharedRef());
		FloatValueMaxForClampingRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetVisibilityOfValueMaxForScoreClamping)));
		FloatValueMaxForClampingRow.ToolTip(LOCTEXT("FloatFilterMaxForClampingRowToolTip", "See Filter Thresholds under the Filter tab.  Values higher than this (before normalization) cause the item to be thrown out as invalid.  Values are normalized with this value as the maximum, so items with this value will have a normalized score of 1."));
		FloatValueMaxForClampingRow.EditCondition(TAttribute<bool>(this, &FEnvQueryTestDetails::AllowWritingToFiltersFromScore), NULL);
	}

	// END Scoring: Clamping, continue Scoring
	//----------------------------

	IDetailPropertyRow& BoolScoreTestRow = ScoreCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, BoolValue)));
	BoolScoreTestRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetBoolValueVisibilityForScoring)));
	BoolScoreTestRow.DisplayName(LOCTEXT("BoolMatchLabel", "Bool Match"));
	BoolScoreTestRow.ToolTip(LOCTEXT("BoolMatchToolTip", "Boolean value to match in order to grant score of 'ScoringFactor'.  Not matching this value will not change score."));

// 	IDetailPropertyRow& ScoreMirrorNormalizedScoreRow = ScoreCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, bMirrorNormalizedScore)));
// 	ScoreMirrorNormalizedScoreRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetFloatScoreVisibility)));

	IDetailPropertyRow& ScoreEquationTypeRow = ScoreCategory.AddProperty(ScoreEquationHandle);
	ScoreEquationTypeRow.CustomWidget()
		.NameContent()
		.VAlign(VAlign_Top)
		[
			ScoreEquationHandle->CreatePropertyNameWidget()
		]
		.ValueContent().MaxDesiredWidth(600)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FEnvQueryTestDetails::OnGetEquationValuesContent)
				.ContentPadding(FMargin( 2.0f, 2.0f ))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FEnvQueryTestDetails::GetEquationValuesDesc)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			+SVerticalBox::Slot()
			.Padding(0, 2, 0, 0)
			.AutoHeight()
			[
				SNew(STextBlock)
				.IsEnabled(false)
				.Text(this, &FEnvQueryTestDetails::GetScoreEquationInfo)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		];
	ScoreEquationTypeRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetFloatScoreVisibility)));

	IDetailPropertyRow& ScoreFactorRow = ScoreCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ScoringFactor)));
	ScoreFactorRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetScoreVisibility)));

	IDetailPropertyRow& ScoreNormalizationTypeRow = ScoreCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, NormalizationType)));
	ScoreNormalizationTypeRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetFloatScoreVisibility)));

	IDetailPropertyRow& ScoreReferenceValueRow = ScoreCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ReferenceValue)));
	ScoreReferenceValueRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetFloatScoreVisibility)));

	IDetailPropertyRow& MultipleContextScoreOpRow = ScoreCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UEnvQueryTest, MultipleContextScoreOp)));
	MultipleContextScoreOpRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvQueryTestDetails::GetScoreVisibility)));

	// scoring & filter function preview
	IDetailCategoryBuilder& PreviewCategory = DetailLayout.EditCategory("Preview");
	PreviewCategory.AddCustomRow(LOCTEXT("Preview", "Preview")).WholeRowWidget
	[
		SAssignNew(PreviewWidget, STestFunctionWidget)
	];

	PreviewWidget->DrawTestOb = Cast<UEnvQueryTest>(MyTest.Get());
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FEnvQueryTestDetails::BuildFilterTestValues()
{
	UEnum* TestConditionEnum = StaticEnum<EEnvTestFilterType::Type>();
	check(TestConditionEnum);

	FilterTestValues.Reset();

	const UEnvQueryTest* EditedTest = Cast<const UEnvQueryTest>(MyTest.Get());
	if (EditedTest)
	{
		if (EditedTest->GetWorkOnFloatValues())
		{
			FilterTestValues.Add(FTextIntPair(TestConditionEnum->GetDisplayNameTextByValue(EEnvTestFilterType::Minimum), EEnvTestFilterType::Minimum));
			FilterTestValues.Add(FTextIntPair(TestConditionEnum->GetDisplayNameTextByValue(EEnvTestFilterType::Maximum), EEnvTestFilterType::Maximum));
			FilterTestValues.Add(FTextIntPair(TestConditionEnum->GetDisplayNameTextByValue(EEnvTestFilterType::Range), EEnvTestFilterType::Range));
		}
		else
		{
			FilterTestValues.Add(FTextIntPair(TestConditionEnum->GetDisplayNameTextByValue(EEnvTestFilterType::Match), EEnvTestFilterType::Match));
		}
	}
}

void FEnvQueryTestDetails::BuildScoreEquationValues()
{
	UEnum* TestScoreEquationEnum = StaticEnum<EEnvTestScoreEquation::Type>();
	check(TestScoreEquationEnum);

	ScoreEquationValues.Reset();

	// Const scoring is always valid.  But other equations are only valid if the score values can be other than boolean values.
	ScoreEquationValues.Add(FTextIntPair(TestScoreEquationEnum->GetDisplayNameTextByValue(EEnvTestScoreEquation::Constant), EEnvTestScoreEquation::Constant));

	const UEnvQueryTest* EditedTest = Cast<const UEnvQueryTest>(MyTest.Get());
	if (EditedTest)
	{
		if (EditedTest->GetWorkOnFloatValues())
		{
			ScoreEquationValues.Add(FTextIntPair(TestScoreEquationEnum->GetDisplayNameTextByValue(EEnvTestScoreEquation::Linear), EEnvTestScoreEquation::Linear));
			ScoreEquationValues.Add(FTextIntPair(TestScoreEquationEnum->GetDisplayNameTextByValue(EEnvTestScoreEquation::Square), EEnvTestScoreEquation::Square));
			ScoreEquationValues.Add(FTextIntPair(TestScoreEquationEnum->GetDisplayNameTextByValue(EEnvTestScoreEquation::InverseLinear), EEnvTestScoreEquation::InverseLinear));
			ScoreEquationValues.Add(FTextIntPair(TestScoreEquationEnum->GetDisplayNameTextByValue(EEnvTestScoreEquation::SquareRoot), EEnvTestScoreEquation::SquareRoot));
		}
	}
}

void FEnvQueryTestDetails::OnFilterTestChange(int32 Index)
{
	const uint8 EnumValue = IntCastChecked<uint8>(Index);
	FilterTypeHandle->SetValue(EnumValue);
}

void FEnvQueryTestDetails::OnScoreEquationChange(int32 Index)
{
	const uint8 EnumValue = IntCastChecked<uint8>(Index);
	ScoreEquationHandle->SetValue(EnumValue);
}

void FEnvQueryTestDetails::BuildScoreClampingTypeValues(bool bBuildMinValues, TArray<FTextIntPair>& ClampTypeValues) const
{
	UEnum* ScoringNormalizationEnum = StaticEnum<EEnvQueryTestClamping::Type>();
	check(ScoringNormalizationEnum);

	ClampTypeValues.Reset();
	ClampTypeValues.Add(FTextIntPair(ScoringNormalizationEnum->GetDisplayNameTextByValue(EEnvQueryTestClamping::None), EEnvQueryTestClamping::None));
	ClampTypeValues.Add(FTextIntPair(ScoringNormalizationEnum->GetDisplayNameTextByValue(EEnvQueryTestClamping::SpecifiedValue), EEnvQueryTestClamping::SpecifiedValue));

	if (IsFiltering())
	{
		bool bSupportFilterThreshold = false;
		if (bBuildMinValues)
		{
			if (UsesFilterMin())
			{
				bSupportFilterThreshold = true;
			}
		}
		else if (UsesFilterMax())
		{
			bSupportFilterThreshold = true;
		}

		if (bSupportFilterThreshold)
		{
			ClampTypeValues.Add(FTextIntPair(ScoringNormalizationEnum->GetDisplayNameTextByValue(EEnvQueryTestClamping::FilterThreshold), EEnvQueryTestClamping::FilterThreshold));
		}
	}
}

void FEnvQueryTestDetails::OnClampMinTestChange(int32 Index)
{
	check(ClampMinTypeHandle.IsValid());
	const uint8 EnumValue = IntCastChecked<uint8>(Index);
	ClampMinTypeHandle->SetValue(EnumValue);
}

void FEnvQueryTestDetails::OnClampMaxTestChange(int32 Index)
{
	check(ClampMaxTypeHandle.IsValid());
	const uint8 EnumValue = IntCastChecked<uint8>(Index);
	ClampMaxTypeHandle->SetValue(EnumValue);
}

TSharedRef<SWidget> FEnvQueryTestDetails::OnGetClampMinTypeContent()
{
 	BuildScoreClampingTypeValues(true, ClampMinTypeValues);
	FMenuBuilder MenuBuilder(true, NULL);

 	for (int32 i = 0; i < ClampMinTypeValues.Num(); i++)
 	{
 		FUIAction ItemAction(FExecuteAction::CreateSP(this, &FEnvQueryTestDetails::OnClampMinTestChange, ClampMinTypeValues[i].Int));
 		MenuBuilder.AddMenuEntry(ClampMinTypeValues[i].Text, TAttribute<FText>(), FSlateIcon(), ItemAction);
 	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FEnvQueryTestDetails::OnGetClampMaxTypeContent()
{
 	BuildScoreClampingTypeValues(false, ClampMaxTypeValues);
	FMenuBuilder MenuBuilder(true, NULL);

 	for (int32 i = 0; i < ClampMaxTypeValues.Num(); i++)
 	{
 		FUIAction ItemAction(FExecuteAction::CreateSP(this, &FEnvQueryTestDetails::OnClampMaxTestChange, ClampMaxTypeValues[i].Int));
 		MenuBuilder.AddMenuEntry(ClampMaxTypeValues[i].Text, TAttribute<FText>(), FSlateIcon(), ItemAction);
 	}

	return MenuBuilder.MakeWidget();
}

FText FEnvQueryTestDetails::GetClampMinTypeDesc() const
{
	check(ClampMinTypeHandle.IsValid());
	uint8 EnumValue;
	ClampMinTypeHandle->GetValue(EnumValue);

	for (int32 i = 0; i < ClampMinTypeValues.Num(); i++)
	{
		if (ClampMinTypeValues[i].Int == EnumValue)
		{
			return ClampMinTypeValues[i].Text;
		}
	}

	return FText::GetEmpty();
}

FText FEnvQueryTestDetails::GetClampMaxTypeDesc() const
{
	check(ClampMaxTypeHandle.IsValid());
	uint8 EnumValue;
	ClampMaxTypeHandle->GetValue(EnumValue);

	for (int32 i = 0; i < ClampMaxTypeValues.Num(); i++)
	{
		if (ClampMaxTypeValues[i].Int == EnumValue)
		{
			return ClampMaxTypeValues[i].Text;
		}
	}

	return FText::GetEmpty();
}

FText FEnvQueryTestDetails::GetEquationValuesDesc() const
{
	check(ScoreEquationHandle.IsValid());
	uint8 EnumValue;
	ScoreEquationHandle->GetValue(EnumValue);

	for (int32 i = 0; i < ScoreEquationValues.Num(); i++)
	{
		if (ScoreEquationValues[i].Int == EnumValue)
		{
			return ScoreEquationValues[i].Text;
		}
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FEnvQueryTestDetails::OnGetFilterTestContent()
{
	BuildFilterTestValues();
	FMenuBuilder MenuBuilder(true, NULL);

	for (int32 i = 0; i < FilterTestValues.Num(); i++)
	{
		FUIAction ItemAction(FExecuteAction::CreateSP(this, &FEnvQueryTestDetails::OnFilterTestChange, FilterTestValues[i].Int));
		MenuBuilder.AddMenuEntry(FilterTestValues[i].Text, TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

FText FEnvQueryTestDetails::GetCurrentFilterTestDesc() const
{
	uint8 EnumValue;
	FilterTypeHandle->GetValue(EnumValue);

	for (int32 i = 0; i < FilterTestValues.Num(); i++)
	{
		if (FilterTestValues[i].Int == EnumValue)
		{
			return FilterTestValues[i].Text;
		}
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FEnvQueryTestDetails::OnGetEquationValuesContent()
{
	BuildScoreEquationValues();
	FMenuBuilder MenuBuilder(true, NULL);

	for (int32 i = 0; i < ScoreEquationValues.Num(); i++)
	{
		FUIAction ItemAction(FExecuteAction::CreateSP(this, &FEnvQueryTestDetails::OnScoreEquationChange, ScoreEquationValues[i].Int));
		MenuBuilder.AddMenuEntry(ScoreEquationValues[i].Text, TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

FText FEnvQueryTestDetails::GetScoreEquationInfo() const
{
	uint8 EnumValue;
	ScoreEquationHandle->GetValue(EnumValue);

	switch (EnumValue)
	{
		case EEnvTestScoreEquation::Linear:
			return LOCTEXT("Linear","Final score = ScoringFactor * NormalizedItemValue");

		case EEnvTestScoreEquation::Square:
			return LOCTEXT("Square","Final score = ScoringFactor * (NormalizedItemValue * NormalizedItemValue)\nBias towards items with big values.");

		case EEnvTestScoreEquation::InverseLinear:
			return LOCTEXT("Inverse","Final score = ScoringFactor * (1.0 - NormalizedItemValue)\nBias towards items with values close to zero.  (Linear, but flipped from 1 to 0 rather than 0 to 1.");

		case EEnvTestScoreEquation::SquareRoot:
			return LOCTEXT("Square root", "Final score = ScoringFactor * Sqrt(NormalizedItemValue)\nNon-linearly bias towards items with big values.");

		case EEnvTestScoreEquation::Constant:
			return LOCTEXT("Constant", "Final score (for values that 'pass') = ScoringFactor\nNOTE: In this case, the score is normally EITHER the ScoringFactor value or zero.\nThe score will be zero if the Normalized Test Value is zero (or if the test value is false for a boolean query).\nOtherwise, score will be the ScoringFactor.");

		default:
			break;
	}

	return FText::GetEmpty();
}

EVisibility FEnvQueryTestDetails::GetVisibilityOfValueMinForScoreClamping() const
{
	if (GetFloatScoreVisibility() == EVisibility::Visible)
	{
		const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
		if ((MyTestOb != NULL) && (MyTestOb->ClampMinType == EEnvQueryTestClamping::FilterThreshold))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetVisibilityOfValueMaxForScoreClamping() const
{
	if (GetFloatScoreVisibility() == EVisibility::Visible)
	{
		const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
		if ((MyTestOb != NULL) && (MyTestOb->ClampMaxType == EEnvQueryTestClamping::FilterThreshold))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetVisibilityOfScoreClampMinimum() const
{
	if (GetFloatScoreVisibility() == EVisibility::Visible)
	{
		const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
		if ((MyTestOb != NULL) && (MyTestOb->ClampMinType == EEnvQueryTestClamping::SpecifiedValue))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetVisibilityOfScoreClampMaximum() const
{
	if (GetFloatScoreVisibility() == EVisibility::Visible)
	{
		const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
		if ((MyTestOb != NULL) && (MyTestOb->ClampMaxType == EEnvQueryTestClamping::SpecifiedValue))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetFloatTestVisibility() const
{
	const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
	if ((MyTestOb != NULL) && MyTestOb->GetWorkOnFloatValues())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

bool FEnvQueryTestDetails::IsFiltering() const
{
	const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
	if (MyTestOb == NULL)
	{
		return false;
	}

	return MyTestOb->IsFiltering();
}

bool FEnvQueryTestDetails::IsScoring() const
{
	const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
	if (MyTestOb == NULL)
	{
		return false;
	}

	return MyTestOb->IsScoring();
}

EVisibility FEnvQueryTestDetails::GetFloatFilterVisibility() const
{
	if (IsFiltering())
	{
		const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
		if (MyTestOb && MyTestOb->GetWorkOnFloatValues())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetScoreVisibility() const
{
	if (IsScoring())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetClampingVisibility() const
{
	if (IsScoring())
	{
		const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
		if (MyTestOb && MyTestOb->GetWorkOnFloatValues())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetFloatScoreVisibility() const
{
	if (IsScoring())
	{
		const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
		if (MyTestOb && MyTestOb->GetWorkOnFloatValues())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

bool FEnvQueryTestDetails::UsesFilterMin() const
{
	uint8 EnumValue;
	check(FilterTypeHandle.IsValid());
	FilterTypeHandle->GetValue(EnumValue);

	return ((EnumValue == EEnvTestFilterType::Minimum) || (EnumValue == EEnvTestFilterType::Range));
}

bool FEnvQueryTestDetails::UsesFilterMax() const
{
	uint8 EnumValue;
	check(FilterTypeHandle.IsValid());
	FilterTypeHandle->GetValue(EnumValue);

	return ((EnumValue == EEnvTestFilterType::Maximum) || (EnumValue == EEnvTestFilterType::Range));
}

EVisibility FEnvQueryTestDetails::GetVisibilityOfFloatValueMin() const
{
	if (IsFiltering())
	{
		uint8 EnumValue;
		FilterTypeHandle->GetValue(EnumValue);

		const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
		if (MyTestOb && MyTestOb->GetWorkOnFloatValues() &&
			((EnumValue == EEnvTestFilterType::Minimum) || (EnumValue == EEnvTestFilterType::Range)))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetVisibilityOfFloatValueMax() const
{
	if (IsFiltering())
	{
		uint8 EnumValue;
		FilterTypeHandle->GetValue(EnumValue);

		const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
		if (MyTestOb && MyTestOb->GetWorkOnFloatValues() &&
			((EnumValue == EEnvTestFilterType::Maximum) || (EnumValue == EEnvTestFilterType::Range)))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

bool FEnvQueryTestDetails::IsMatchingBoolValue() const
{
	// TODO: Decide whether this complex function needs to be so complex!  At the moment, if it's not working on floats,
	// it must be working on bools, in which case it MUST be using a Match test.  So probably it could stop much earlier!
	const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
	if (MyTestOb && !MyTestOb->GetWorkOnFloatValues())
	{
		if (FilterTypeHandle.IsValid())
		{
			uint8 EnumValue;
			FilterTypeHandle->GetValue(EnumValue);
			if (EnumValue == EEnvTestFilterType::Match)
			{
				return true;
			}
		}
	}

	return false;
}
EVisibility FEnvQueryTestDetails::GetVisibilityForFiltering() const
{
	if (IsFiltering())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetBoolValueVisibilityForScoring() const
{
	if (!IsFiltering())
	{
		if (IsMatchingBoolValue())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetBoolValueVisibility() const
{
	if (IsFiltering())
	{
		if (IsMatchingBoolValue())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvQueryTestDetails::GetTestPreviewVisibility() const
{
	const UEnvQueryTest* MyTestOb = Cast<const UEnvQueryTest>(MyTest.Get());
	if ((MyTestOb != NULL) && MyTestOb->GetWorkOnFloatValues())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
