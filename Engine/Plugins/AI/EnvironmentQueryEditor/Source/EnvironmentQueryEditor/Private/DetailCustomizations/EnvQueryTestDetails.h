// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "UObject/WeakObjectPtr.h"
#include "IDetailCustomization.h"

struct EVisibility;

class IDetailLayoutBuilder;
class IPropertyHandle;
class STestFunctionWidget;
class SWidget;

class FEnvQueryTestDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

protected:

	/** cached name, value pairs of enums that may change available options based on other options. */
	struct FTextIntPair
	{
		FText Text;
		int32 Int;

		FTextIntPair() {}
		FTextIntPair(FText InText, int32 InInt) : Text(InText), Int(InInt) {}
	};

	TSharedPtr<IPropertyHandle> FilterTypeHandle;
	TSharedPtr<IPropertyHandle> ScoreEquationHandle;
	TSharedPtr<IPropertyHandle> TestPurposeHandle;
	TSharedPtr<IPropertyHandle> ClampMinTypeHandle;
	TSharedPtr<IPropertyHandle> ClampMaxTypeHandle;
	TSharedPtr<IPropertyHandle> ScoreClampMinHandle;
	TSharedPtr<IPropertyHandle> FloatValueMinHandle;
	TSharedPtr<IPropertyHandle> ScoreClampMaxHandle;
	TSharedPtr<IPropertyHandle> FloatValueMaxHandle;
	TSharedPtr<IPropertyHandle> MultipleContextFilterOpHandle;
	TSharedPtr<IPropertyHandle> ScoreHandle;
	TSharedPtr<IPropertyHandle> ScoreNormalizationTypeHandle;
	TSharedPtr<IPropertyHandle> ScoreReferenceValueHandle;
	TSharedPtr<IPropertyHandle> MultipleContextScoreOpHandle;

	bool IsFiltering() const;
	bool IsScoring() const;

	bool UsesFilterMin() const;
	bool UsesFilterMax() const;

	FText GetCurrentFilterTestDesc() const;
	FText GetScoreEquationInfo() const;

 	EVisibility GetScoreVisibility() const;
	EVisibility GetClampingVisibility() const;

	TSharedRef<SWidget> OnGetFilterTestContent();
	void BuildFilterTestValues();

	void BuildScoreEquationValues();
	TSharedRef<SWidget> OnGetEquationValuesContent();
	FText GetEquationValuesDesc() const;
	void OnScoreEquationChange(int32 Index);

	void OnFilterTestChange(int32 Index);
	void OnClampMinTestChange(int32 Index);
	void OnClampMaxTestChange(int32 Index);

	TSharedRef<SWidget> OnGetClampMaxTypeContent();
	FText GetClampMaxTypeDesc() const;

	TSharedRef<SWidget> OnGetClampMinTypeContent();
	FText GetClampMinTypeDesc() const;

	bool IsMatchingBoolValue() const;

	// Is this a float test at all?
	EVisibility GetFloatTestVisibility() const;
	
	// Is this a float test that is filtering?
	EVisibility GetFloatFilterVisibility() const;

	// Is this a float test that is scoring?
	EVisibility GetFloatScoreVisibility() const;
	
	EVisibility GetTestPreviewVisibility() const;
	EVisibility GetVisibilityOfFloatValueMin() const;
	EVisibility GetVisibilityOfFloatValueMax() const;
	EVisibility GetVisibilityOfValueMinForScoreClamping() const;
	EVisibility GetVisibilityOfValueMaxForScoreClamping() const;
	EVisibility GetVisibilityForFiltering() const;
	EVisibility GetBoolValueVisibilityForScoring() const;
	EVisibility GetBoolValueVisibility() const;
	EVisibility GetVisibilityOfScoreClampMinimum() const;
	EVisibility GetVisibilityOfScoreClampMaximum() const;

	void BuildScoreClampingTypeValues(bool bBuildMinValues, TArray<FTextIntPair>& ClampTypeValues) const;
	
	void UpdateTestFunctionPreview() const;
	void FillEquationSamples(uint8 EquationType, bool bInversed, TArray<float>& Samples) const;
	TSharedPtr<STestFunctionWidget> PreviewWidget;


	TArray<FTextIntPair> FilterTestValues;

	TArray<FTextIntPair> ClampMinTypeValues;
	TArray<FTextIntPair> ClampMaxTypeValues;

	TArray<FTextIntPair> ScoreEquationValues;

	TWeakObjectPtr<UObject> MyTest;

	FORCEINLINE bool AllowWritingToFiltersFromScore() const { return false; }
	TAttribute<bool> AllowWriting;
};
