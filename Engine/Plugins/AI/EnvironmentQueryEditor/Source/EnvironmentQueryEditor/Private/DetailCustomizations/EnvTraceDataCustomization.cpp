// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations/EnvTraceDataCustomization.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SComboButton.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "EnvTraceDataCustomization"

TSharedRef<IPropertyTypeCustomization> FEnvTraceDataCustomization::MakeInstance()
{
	return MakeShareable( new FEnvTraceDataCustomization );
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FEnvTraceDataCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Text(this, &FEnvTraceDataCustomization::GetShortDescription)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];

	PropTraceMode = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,TraceMode));
	PropTraceShape = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,TraceShape));
	PropTraceChannel = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData, TraceChannel));
	PropTraceChannelSerialized = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData, SerializedChannel));
	PropTraceProfileName = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData, TraceProfileName));
	CacheTraceModes(StructPropertyHandle);
}

void FEnvTraceDataCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	if (TraceModes.Num() > 1)
	{
		StructBuilder.AddCustomRow(LOCTEXT("TraceMode", "Trace Mode"))
		.NameContent()
		[
			PropTraceMode->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FEnvTraceDataCustomization::OnGetTraceModeContent)
			.ContentPadding(FMargin( 2.0f, 2.0f ))
			.ButtonContent()
			[
				SNew(STextBlock) 
				.Text(this, &FEnvTraceDataCustomization::GetCurrentTraceModeDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}

	// navmesh props
	TSharedPtr<IPropertyHandle> PropNavFilter = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,NavigationFilter));
	StructBuilder.AddProperty(PropNavFilter.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetNavigationVisibility)));

	// geometry props
	PropTraceChannel->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FEnvTraceDataCustomization::OnTraceChannelChanged));
	StructBuilder.AddProperty(PropTraceChannel.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetGeometryByChannelVisibility)));

	StructBuilder.AddProperty(PropTraceShape.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetGeometryVisibility)));

	StructBuilder.AddProperty(PropTraceProfileName.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetGeometryByProfileVisibility)));

	// common props
	TSharedPtr<IPropertyHandle> PropExtX = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,ExtentX));
	StructBuilder.AddProperty(PropExtX.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetExtentX)));

	TSharedPtr<IPropertyHandle> PropExtY = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,ExtentY));
	StructBuilder.AddProperty(PropExtY.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetExtentY)));

	TSharedPtr<IPropertyHandle> PropExtZ = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,ExtentZ));
	StructBuilder.AddProperty(PropExtZ.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetExtentZ)));

	// projection props
	TSharedPtr<IPropertyHandle> PropHeightDown = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,ProjectDown));
	StructBuilder.AddProperty(PropHeightDown.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetProjectionVisibility)));

	TSharedPtr<IPropertyHandle> PropHeightUp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,ProjectUp));
	StructBuilder.AddProperty(PropHeightUp.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetProjectionVisibility)));

	// advanced props
	TSharedPtr<IPropertyHandle> PropPostProjectionVerticalOffset = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData, PostProjectionVerticalOffset));
	StructBuilder.AddProperty(PropPostProjectionVerticalOffset.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetProjectionVisibility)));

	TSharedPtr<IPropertyHandle> PropTraceComplex = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,bTraceComplex));
	StructBuilder.AddProperty(PropTraceComplex.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetGeometryVisibility)));

	TSharedPtr<IPropertyHandle> PropOnlyBlocking = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData,bOnlyBlockingHits));
	StructBuilder.AddProperty(PropOnlyBlocking.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FEnvTraceDataCustomization::GetGeometryVisibility)));
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FEnvTraceDataCustomization::CacheTraceModes(TSharedRef<class IPropertyHandle> StructPropertyHandle)
{
	TSharedPtr<IPropertyHandle> PropCanNavMesh = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData, bCanTraceOnNavMesh));
	TSharedPtr<IPropertyHandle> PropCanGeometry = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData, bCanTraceOnGeometry));
	TSharedPtr<IPropertyHandle> PropCanDisable = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData, bCanDisableTrace));
	TSharedPtr<IPropertyHandle> PropCanProject = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvTraceData, bCanProjectDown));

	bool bCanNavMesh = false;
	bool bCanGeometry = false;
	bool bCanDisable = false;
	bCanShowProjection = false;
	PropCanNavMesh->GetValue(bCanNavMesh);
	PropCanGeometry->GetValue(bCanGeometry);
	PropCanDisable->GetValue(bCanDisable);
	PropCanProject->GetValue(bCanShowProjection);

	static UEnum* TraceModeEnum = StaticEnum<EEnvQueryTrace::Type>();
	check(TraceModeEnum);

	TraceModes.Reset();
	if (bCanDisable)
	{
		TraceModes.Add(FTextIntPair(TraceModeEnum->GetDisplayNameTextByValue(EEnvQueryTrace::None), EEnvQueryTrace::None));
	}
	if (bCanNavMesh)
	{
		TraceModes.Add(FTextIntPair(TraceModeEnum->GetDisplayNameTextByValue(EEnvQueryTrace::Navigation), EEnvQueryTrace::Navigation));
	}
	if (bCanGeometry)
	{
		TraceModes.Add(FTextIntPair(TraceModeEnum->GetDisplayNameTextByValue(EEnvQueryTrace::GeometryByChannel), EEnvQueryTrace::GeometryByChannel));
		TraceModes.Add(FTextIntPair(TraceModeEnum->GetDisplayNameTextByValue(EEnvQueryTrace::GeometryByProfile), EEnvQueryTrace::GeometryByProfile));
	}
	if (bCanGeometry && bCanNavMesh && !bCanShowProjection)
	{
		TraceModes.Add(FTextIntPair(TraceModeEnum->GetDisplayNameTextByValue(EEnvQueryTrace::NavigationOverLedges), EEnvQueryTrace::NavigationOverLedges));
	}

	ActiveMode = EEnvQueryTrace::None;
	PropTraceMode->GetValue(ActiveMode);
}

void FEnvTraceDataCustomization::OnTraceChannelChanged()
{
	uint8 TraceChannelValue;
	FPropertyAccess::Result Result = PropTraceChannel->GetValue(TraceChannelValue);
	if (Result == FPropertyAccess::Success)
	{
		ETraceTypeQuery TraceTypeValue = (ETraceTypeQuery)TraceChannelValue;
		ECollisionChannel CollisionChannelValue = UEngineTypes::ConvertToCollisionChannel(TraceTypeValue);

		uint8 SerializedChannelValue = IntCastChecked<uint8>((int)CollisionChannelValue);
		PropTraceChannelSerialized->SetValue(SerializedChannelValue);
	}
}

void FEnvTraceDataCustomization::OnTraceModeChanged(int32 Index)
{
	ActiveMode = IntCastChecked<uint8>(Index);
	PropTraceMode->SetValue(ActiveMode);
}

TSharedRef<SWidget> FEnvTraceDataCustomization::OnGetTraceModeContent()
{
	FMenuBuilder MenuBuilder(true, NULL);

	for (int32 i = 0; i < TraceModes.Num(); i++)
	{
		FUIAction ItemAction( FExecuteAction::CreateSP( this, &FEnvTraceDataCustomization::OnTraceModeChanged, TraceModes[i].Int ) );
		MenuBuilder.AddMenuEntry( TraceModes[i].Text, TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

FText FEnvTraceDataCustomization::GetCurrentTraceModeDesc() const
{
	for (int32 i = 0; i < TraceModes.Num(); i++)
	{
		if (TraceModes[i].Int == ActiveMode)
		{
			return TraceModes[i].Text;
		}
	}

	return FText::GetEmpty();
}

FText FEnvTraceDataCustomization::GetShortDescription() const
{
	FText Desc = FText::GetEmpty();

	switch (ActiveMode)
	{
	case EEnvQueryTrace::GeometryByChannel:
		Desc = LOCTEXT("TraceGeomByChannel","geometry trace by channel");
		break;

	case EEnvQueryTrace::GeometryByProfile:
		Desc = LOCTEXT("TraceGeomByProfile", "geometry trace by profile");
		break;

	case EEnvQueryTrace::Navigation:
		Desc = LOCTEXT("TraceNav","navmesh trace");
		break;

	case EEnvQueryTrace::NavigationOverLedges:
		Desc = LOCTEXT("TraceNavAndGeo", "navmesh trace, ignore hitting ledges");
		break;

	case EEnvQueryTrace::None:
		Desc = LOCTEXT("TraceNone","trace disabled");
		break;

	default: break;
	}

	return Desc;
}

EVisibility FEnvTraceDataCustomization::GetGeometryByChannelVisibility() const
{
	return (ActiveMode == EEnvQueryTrace::GeometryByChannel || ActiveMode == EEnvQueryTrace::NavigationOverLedges) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FEnvTraceDataCustomization::GetGeometryByProfileVisibility() const
{
	return (ActiveMode == EEnvQueryTrace::GeometryByProfile) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FEnvTraceDataCustomization::GetGeometryVisibility() const
{
	return (ActiveMode == EEnvQueryTrace::GeometryByChannel || ActiveMode == EEnvQueryTrace::GeometryByProfile || ActiveMode == EEnvQueryTrace::NavigationOverLedges) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FEnvTraceDataCustomization::GetNavigationVisibility() const
{
	return (ActiveMode == EEnvQueryTrace::Navigation || ActiveMode == EEnvQueryTrace::NavigationOverLedges) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FEnvTraceDataCustomization::GetProjectionVisibility() const
{
	return (ActiveMode != EEnvQueryTrace::None) && bCanShowProjection ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FEnvTraceDataCustomization::GetExtentX() const
{
	if (ActiveMode == EEnvQueryTrace::Navigation || (ActiveMode == EEnvQueryTrace::NavigationOverLedges && bCanShowProjection == false))
	{
		// radius
		return EVisibility::Visible;
	}
	else if (ActiveMode == EEnvQueryTrace::GeometryByChannel || ActiveMode == EEnvQueryTrace::GeometryByProfile)
	{
		uint8 EnumValue;
		PropTraceShape->GetValue(EnumValue);

		return (EnumValue != EEnvTraceShape::Line) ? EVisibility::Visible : EVisibility::Collapsed;
	}
		
	return EVisibility::Collapsed;
}

EVisibility FEnvTraceDataCustomization::GetExtentY() const
{
	if (ActiveMode == EEnvQueryTrace::GeometryByChannel || ActiveMode == EEnvQueryTrace::GeometryByProfile || (ActiveMode == EEnvQueryTrace::NavigationOverLedges && bCanShowProjection == false))
	{
		uint8 EnumValue;
		PropTraceShape->GetValue(EnumValue);

		return (EnumValue == EEnvTraceShape::Box) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

EVisibility FEnvTraceDataCustomization::GetExtentZ() const
{
	if (ActiveMode == EEnvQueryTrace::GeometryByChannel || ActiveMode == EEnvQueryTrace::GeometryByProfile || (ActiveMode == EEnvQueryTrace::NavigationOverLedges && bCanShowProjection == false))
	{
		uint8 EnumValue;
		PropTraceShape->GetValue(EnumValue);

		return (EnumValue == EEnvTraceShape::Box || EnumValue == EEnvTraceShape::Capsule) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
