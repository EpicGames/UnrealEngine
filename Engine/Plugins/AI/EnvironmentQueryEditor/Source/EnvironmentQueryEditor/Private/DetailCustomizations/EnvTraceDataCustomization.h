// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class SWidget;
struct EVisibility;

class IPropertyHandle;

class FEnvTraceDataCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization Interface
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	//~ End IPropertyTypeCustomization Interface

protected:

	void CacheTraceModes(TSharedRef<class IPropertyHandle> StructPropertyHandle);

	/** cached names of trace modes */
	struct FTextIntPair
	{
		FText Text;
		int32 Int;

		FTextIntPair() {}
		FTextIntPair(FText InText, int32 InInt) : Text(InText), Int(InInt) {}
	};
	TArray<FTextIntPair> TraceModes;
	bool bCanShowProjection;
	uint8 ActiveMode;

	TSharedPtr<IPropertyHandle> PropTraceMode;
	TSharedPtr<IPropertyHandle> PropTraceShape;
	TSharedPtr<IPropertyHandle> PropTraceChannel;
	TSharedPtr<IPropertyHandle> PropTraceChannelSerialized;
	TSharedPtr<IPropertyHandle> PropTraceProfileName;

	void OnTraceChannelChanged();
	void OnTraceProfileChanged();
	void OnTraceModeChanged(int32 Index);
	TSharedRef<SWidget> OnGetTraceModeContent();
	FText GetCurrentTraceModeDesc() const;
	FText GetShortDescription() const;

	EVisibility GetGeometryByChannelVisibility() const;
	EVisibility GetGeometryByProfileVisibility() const;
	EVisibility GetGeometryVisibility() const;
	EVisibility GetNavigationVisibility() const;
	EVisibility GetProjectionVisibility() const;
	EVisibility GetExtentX() const;
	EVisibility GetExtentY() const;
	EVisibility GetExtentZ() const;
};
