// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphSchema_EnvironmentQuery.h"
#include "Modules/ModuleManager.h"
#include "EdGraph/EdGraph.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvironmentQueryGraphNode_Option.h"
#include "EnvironmentQueryGraphNode_Root.h"
#include "EnvironmentQueryGraphNode_Test.h"
#include "EnvironmentQueryEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphSchema_EnvironmentQuery)

#define LOCTEXT_NAMESPACE "EnvironmentQueryEditor"

//////////////////////////////////////////////////////////////////////////

UEdGraphSchema_EnvironmentQuery::UEdGraphSchema_EnvironmentQuery(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UEdGraphSchema_EnvironmentQuery::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	FGraphNodeCreator<UEnvironmentQueryGraphNode_Root> NodeCreator(Graph);
	UEnvironmentQueryGraphNode_Root* MyNode = NodeCreator.CreateNode();
	NodeCreator.Finalize();
	SetNodeMetaData(MyNode, FNodeMetadata::DefaultGraphNode);
}

void UEdGraphSchema_EnvironmentQuery::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	UEnvironmentQueryGraphNode* ParentGraphNode = ContextMenuBuilder.FromPin ? Cast<UEnvironmentQueryGraphNode>(ContextMenuBuilder.FromPin->GetOuter()) : NULL;
	if  (ParentGraphNode && !ParentGraphNode->IsA(UEnvironmentQueryGraphNode_Root::StaticClass()))
	{
		return;
	}

	FEnvironmentQueryEditorModule& EditorModule = FModuleManager::GetModuleChecked<FEnvironmentQueryEditorModule>(TEXT("EnvironmentQueryEditor"));
	FGraphNodeClassHelper* ClassCache = EditorModule.GetClassCache().Get();
	
	TArray<FGraphNodeClassData> NodeClasses;
	ClassCache->GatherClasses(UEnvQueryGenerator::StaticClass(), NodeClasses);

	FCategorizedGraphActionListBuilder GeneratorsBuilder(TEXT("Generators"));
	for (const auto& NodeClass : NodeClasses)
	{
		const FText NodeTypeName = FText::FromString(FName::NameToDisplayString(NodeClass.ToString(), false));

		UEnvironmentQueryGraphNode_Option* OpNode = NewObject<UEnvironmentQueryGraphNode_Option>(ContextMenuBuilder.OwnerOfTemporaries);
		OpNode->ClassData = NodeClass;

		TSharedPtr<FAISchemaAction_NewNode> AddOpAction = AddNewNodeAction(GeneratorsBuilder, NodeClass.GetCategory(), NodeTypeName, FText::GetEmpty());
		AddOpAction->NodeTemplate = OpNode;
	}

	ContextMenuBuilder.Append(GeneratorsBuilder);
}

void UEdGraphSchema_EnvironmentQuery::GetSubNodeClasses(int32 SubNodeFlags, TArray<FGraphNodeClassData>& ClassData, UClass*& GraphNodeClass) const
{
	FEnvironmentQueryEditorModule& EditorModule = FModuleManager::GetModuleChecked<FEnvironmentQueryEditorModule>(TEXT("EnvironmentQueryEditor"));
	FGraphNodeClassHelper* ClassCache = EditorModule.GetClassCache().Get();

	ClassCache->GatherClasses(UEnvQueryTest::StaticClass(), ClassData);
	GraphNodeClass = UEnvironmentQueryGraphNode_Test::StaticClass();
}

const FPinConnectionResponse UEdGraphSchema_EnvironmentQuery::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Both are on the same node"));
	}

	if ((PinA->Direction == EGPD_Input && PinA->LinkedTo.Num()>0) || 
		(PinB->Direction == EGPD_Input && PinB->LinkedTo.Num()>0))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
	}

	// Compare the directions
	bool bDirectionsOK = false;

	if ((PinA->Direction == EGPD_Input) && (PinB->Direction == EGPD_Output))
	{
		bDirectionsOK = true;
	}
	else if ((PinB->Direction == EGPD_Input) && (PinA->Direction == EGPD_Output))
	{
		bDirectionsOK = true;
	}

	if (bDirectionsOK)
	{
		if ( (PinA->Direction == EGPD_Input && PinA->LinkedTo.Num()>0) || (PinB->Direction == EGPD_Input && PinB->LinkedTo.Num()>0))
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Already connected with other"));
		}
	}
	else
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}

const FPinConnectionResponse UEdGraphSchema_EnvironmentQuery::CanMergeNodes(const UEdGraphNode* NodeA, const UEdGraphNode* NodeB) const
{
	// Make sure the nodes are not the same 
	if (NodeA == NodeB)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Both are the same node"));
	}

	const bool bNodeAIsTest = NodeA->IsA(UEnvironmentQueryGraphNode_Test::StaticClass());
	const bool bNodeAIsOption = NodeA->IsA(UEnvironmentQueryGraphNode_Option::StaticClass());
	const bool bNodeBIsTest = NodeB->IsA(UEnvironmentQueryGraphNode_Test::StaticClass());
	const bool bNodeBIsOption = NodeB->IsA(UEnvironmentQueryGraphNode_Option::StaticClass());

	if (bNodeAIsTest && (bNodeBIsOption || bNodeBIsTest))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
}

#undef LOCTEXT_NAMESPACE

