// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"

class Error;

namespace EnvironmentQueryColors
{
	namespace NodeBody
	{
		const FLinearColor Default(0.15f, 0.15f, 0.15f);
		const FLinearColor Generator(0.1f, 0.1f, 0.1f);
		const FLinearColor Test(0.0f, 0.07f, 0.4f);
		const FLinearColor TestInactive(0.1f, 0.1f, 0.1f);
		const FLinearColor Error(1.0f, 0.0f, 0.0f);
	}

	namespace NodeBorder
	{
		const FLinearColor Default(0.08f, 0.08f, 0.08f);
		const FLinearColor Selected(1.00f, 0.08f, 0.08f);
	}

	namespace Pin
	{
		const FLinearColor Default(0.02f, 0.02f, 0.02f);
		const FLinearColor Hover(1.0f, 0.7f, 0.0f);
	}

	namespace Action
	{
		const FLinearColor DragMarker(1.0f, 1.0f, 0.2f);
		const FLinearColor Weight(0.0f, 1.0f, 1.0f);
		const FLinearColor WeightNamed(0.2f, 0.2f, 0.2f);
		const FLinearColor Profiler(0.1f, 0.1f, 0.1f, 1.0f);
	}
}
