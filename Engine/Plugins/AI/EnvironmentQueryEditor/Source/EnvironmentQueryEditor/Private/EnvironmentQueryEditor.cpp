// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQueryEditor.h"
#include "DetailsViewArgs.h"
#include "EdGraph/EdGraphSchema.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "EnvironmentQuery/EnvQueryOption.h"
#include "EnvironmentQueryGraph.h"
#include "EnvironmentQueryGraphNode_Option.h"
#include "EnvironmentQueryGraphNode_Root.h"
#include "EnvironmentQueryGraphNode_Test.h"
#include "IDesktopPlatform.h"
#include "PropertyEditorModule.h"
#include "EnvironmentQueryEditorModule.h"
#include "Misc/Paths.h"
#include "SEnvQueryProfiler.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Application/SlateApplication.h"
#include "DesktopPlatformModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "EnvironmentQuery/EnvQuery.h"

#include "IDetailsView.h"
#include "Widgets/Docking/SDockTab.h"
 
#define LOCTEXT_NAMESPACE "EnvironmentQueryEditor"

namespace FEnvironmentQueryHelper
{
	static const FString StatFileDescription = LOCTEXT("FileTypeDescription", "EQS Stat File").ToString();
	static const FString LoadFileTypes = FString::Printf(TEXT("%s (*.ue_eqs)|*.ue_eqs"), *StatFileDescription);
	static const FString SaveFileTypes = FString::Printf(TEXT("%s (*.ue_eqs)|*.ue_eqs"), *StatFileDescription);
}

const FName FEnvironmentQueryEditor::EQSUpdateGraphTabId(TEXT("EnvironmentQueryEditor_UpdateGraph"));
const FName FEnvironmentQueryEditor::EQSPropertiesTabId(TEXT("EnvironmentQueryEditor_Properties"));
const FName FEnvironmentQueryEditor::EQSProfilerTabId(TEXT("EnvironmentQueryEditor_Profiler"));

class FEnvQueryCommands : public TCommands<FEnvQueryCommands>
{
public:
	FEnvQueryCommands()	: TCommands<FEnvQueryCommands>("EnvQueryEditor.Profiler", LOCTEXT("Profiler", "Profiler"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	TSharedPtr<FUICommandInfo> LoadStats;
	TSharedPtr<FUICommandInfo> SaveStats;

	/** Initialize commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(LoadStats, "Load Stats", "Load EQS Profiler stats", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SaveStats, "Save Stats", "Save EQS Profiler stats", EUserInterfaceActionType::Button, FInputChord());
	}
};

void FEnvironmentQueryEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_EnvironmentQueryEditor", "Environment Query Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( EQSUpdateGraphTabId, FOnSpawnTab::CreateSP(this, &FEnvironmentQueryEditor::SpawnTab_UpdateGraph) )
		.SetDisplayName( NSLOCTEXT("EnvironmentQueryEditor", "Graph", "Graph") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

	InTabManager->RegisterTabSpawner( EQSPropertiesTabId, FOnSpawnTab::CreateSP(this, &FEnvironmentQueryEditor::SpawnTab_Properties) )
		.SetDisplayName( NSLOCTEXT("EnvironmentQueryEditor", "PropertiesTab", "Details" ) )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(EQSProfilerTabId, FOnSpawnTab::CreateSP(this, &FEnvironmentQueryEditor::SpawnTab_Profiler))
		.SetDisplayName(NSLOCTEXT("EnvironmentQueryEditor", "ProfilerTab", "Profiler"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Profiler.EventGraph.ExpandHotPath"));
}

void FEnvironmentQueryEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(EQSPropertiesTabId);
	InTabManager->UnregisterTabSpawner(EQSUpdateGraphTabId);
	InTabManager->UnregisterTabSpawner(EQSProfilerTabId);
}

void FEnvironmentQueryEditor::InitEnvironmentQueryEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UEnvQuery* InScript )
{
	Query = InScript;
	check(Query != NULL);

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_EnvironmentQuery_Layout_v2" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab( EQSUpdateGraphTabId, ETabState::OpenedTab )
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.3f)
				->AddTab( EQSPropertiesTabId, ETabState::OpenedTab )
				->AddTab( EQSProfilerTabId, ETabState::OpenedTab)
				->SetForegroundTab(EQSPropertiesTabId)
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, FEnvironmentQueryEditorModule::EnvironmentQueryEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, Query );
	
	FEnvironmentQueryEditorModule& EnvironmentQueryEditorModule = FModuleManager::LoadModuleChecked<FEnvironmentQueryEditorModule>( "EnvironmentQueryEditor" );
	AddMenuExtender(EnvironmentQueryEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	AddToolbarExtender(EnvironmentQueryEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	BindCommands();
	ExtendToolbar();
	RegenerateMenusAndToolbars();

	// Update BT asset data based on saved graph to have correct data in editor
	TSharedPtr<SGraphEditor> UpdateGraphEditor = UpdateGraphEdPtr.Pin();
	if (UpdateGraphEditor.IsValid() && UpdateGraphEditor->GetCurrentGraph() != NULL)
	{
		//let's find root node
		UEnvironmentQueryGraph* EQSGraph = Cast<UEnvironmentQueryGraph>(UpdateGraphEditor->GetCurrentGraph());
		EQSGraph->UpdateAsset();
	}
}

FName FEnvironmentQueryEditor::GetToolkitFName() const
{
	return FName("Environment Query");
}

FText FEnvironmentQueryEditor::GetBaseToolkitName() const
{
	return NSLOCTEXT("EnvironmentQueryEditor", "AppLabel", "EnvironmentQuery");
}

FString FEnvironmentQueryEditor::GetWorldCentricTabPrefix() const
{
	return NSLOCTEXT("EnvironmentQueryEditor", "WorldCentricTabPrefix", "EnvironmentQuery ").ToString();
}


FLinearColor FEnvironmentQueryEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}


/** Create new tab for the supplied graph - don't call this directly, call SExplorer->FindTabForGraph.*/
TSharedRef<SGraphEditor> FEnvironmentQueryEditor::CreateGraphEditorWidget(UEdGraph* InGraph)
{
	check(InGraph != NULL);
	
	// Create the appearance info
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = NSLOCTEXT("EnvironmentQueryEditor", "AppearanceCornerText", "ENVIRONMENT QUERY");

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FEnvironmentQueryEditor::OnSelectedNodesChanged);
	
	CreateCommandList();

	// Make title bar
	TSharedRef<SWidget> TitleBarWidget = 
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush( TEXT("Graph.TitleBackground") ) )
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("EnvironmentQueryEditor", "TheQueryGraphLabel", "Query Graph"))
				.TextStyle( FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText") )
			]
		];

	// Make full graph editor
	return SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.Appearance(AppearanceInfo)
		.TitleBar(TitleBarWidget)
		.GraphToEdit(InGraph)
		.GraphEvents(InEvents);
}


TSharedRef<SDockTab> FEnvironmentQueryEditor::SpawnTab_UpdateGraph( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == EQSUpdateGraphTabId );
	UEnvironmentQueryGraph* MyGraph = Cast<UEnvironmentQueryGraph>(Query->EdGraph);
	if (Query->EdGraph == NULL)
	{
		MyGraph = NewObject<UEnvironmentQueryGraph>(Query, NAME_None, RF_Transactional);
		Query->EdGraph = MyGraph;

		// let's read data from BT script and generate nodes
		const UEdGraphSchema* Schema = Query->EdGraph->GetSchema();
		Schema->CreateDefaultNodesForGraph(*Query->EdGraph);

		MyGraph->OnCreated();
	}
	else
	{
		MyGraph->OnLoaded();
	}

	MyGraph->Initialize();

	TSharedRef<SGraphEditor> UpdateGraphEditor = CreateGraphEditorWidget(Query->EdGraph);
	UpdateGraphEdPtr = UpdateGraphEditor; // Keep pointer to editor

	return SNew(SDockTab)
		.Label( NSLOCTEXT("EnvironmentQueryEditor", "UpdateGraph", "Update Graph") )
		.TabColorScale( GetTabColorScale() )
		[
			UpdateGraphEditor
		];
}

TSharedRef<SDockTab> FEnvironmentQueryEditor::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check( Args.GetTabId() == EQSPropertiesTabId );

	CreateInternalWidgets();

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(NSLOCTEXT("EnvironmentQueryEditor", "PropertiesTab", "Details"))
		[
			DetailsView.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FEnvironmentQueryEditor::SpawnTab_Profiler(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == EQSProfilerTabId);

	ProfilerView = SNew(SEnvQueryProfiler)
		.OwnerQueryName(Query ? Query->GetFName() : NAME_None)
		.OnDataChanged(FSimpleDelegate::CreateSP(this, &FEnvironmentQueryEditor::OnStatsDataChange));

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(NSLOCTEXT("EnvironmentQueryEditor", "ProfilerTab", "Profiler"))
		[
			ProfilerView.ToSharedRef()
		];

	return SpawnedTab;
}

void FEnvironmentQueryEditor::OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection)
{
	TArray<UObject*> Selection;

	if (NewSelection.Num())
	{
		for(TSet<class UObject*>::TConstIterator SetIt(NewSelection);SetIt;++SetIt)
		{
			UEnvironmentQueryGraphNode* GraphNode = Cast<UEnvironmentQueryGraphNode>(*SetIt);
			if (GraphNode)
			{
				if (GraphNode->IsA(UEnvironmentQueryGraphNode_Root::StaticClass()))
				{
					Selection.Add(GraphNode);
				}
				else if (GraphNode->IsA(UEnvironmentQueryGraphNode_Option::StaticClass()))
				{
					UEnvQueryOption* QueryOption = Cast<UEnvQueryOption>(GraphNode->NodeInstance);
					if (QueryOption)
					{
						Selection.Add(QueryOption->Generator);
					}
				}
				else
				{
					Selection.Add(GraphNode->NodeInstance);
				}
			}
		}
	}

	TSharedPtr<FTabManager> TabManagerPtr = GetTabManager();
	if (TabManagerPtr)
	{
		TabManagerPtr->TryInvokeTab(EQSPropertiesTabId);
	}
	
	if (DetailsView)
	{
		if (Selection.Num() == 1)
		{
			DetailsView->SetObjects(Selection);
		}
		else
		{
			DetailsView->SetObject(NULL);
		}
	}
}

void FEnvironmentQueryEditor::CreateInternalWidgets()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	DetailsView->SetObject( nullptr );
	DetailsView->OnFinishedChangingProperties().AddSP(this, &FEnvironmentQueryEditor::OnFinishedChangingProperties);
}

void FEnvironmentQueryEditor::SaveAsset_Execute()
{
	// modify BT asset
	TSharedPtr<SGraphEditor> UpdateGraphEditor = UpdateGraphEdPtr.Pin();
	if (UpdateGraphEditor.IsValid() && UpdateGraphEditor->GetCurrentGraph() != NULL)
	{
		//let's find root node
		UEnvironmentQueryGraph* EdGraph = Cast<UEnvironmentQueryGraph>(UpdateGraphEditor->GetCurrentGraph());
		EdGraph->UpdateAsset();
	}
	// save it
	IEnvironmentQueryEditor::SaveAsset_Execute();
}

void FEnvironmentQueryEditor::BindCommands()
{
	FEnvQueryCommands::Register();

	ToolkitCommands->MapAction(FEnvQueryCommands::Get().LoadStats,
		FExecuteAction::CreateSP(this, &FEnvironmentQueryEditor::OnLoadStats)
		);

	ToolkitCommands->MapAction(FEnvQueryCommands::Get().SaveStats,
		FExecuteAction::CreateSP(this, &FEnvironmentQueryEditor::OnSaveStats)
		);
}

void FEnvironmentQueryEditor::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("Profiler");
			{
				ToolbarBuilder.AddToolBarButton(FEnvQueryCommands::Get().LoadStats);
				ToolbarBuilder.AddToolBarButton(FEnvQueryCommands::Get().SaveStats);
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		ToolkitCommands,
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar)
		);

	AddToolbarExtender(ToolbarExtender);
}

void FEnvironmentQueryEditor::OnSaveStats()
{
	TArray<FString> SaveFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bSaved = false;
	if (DesktopPlatform)
	{
		const FString DefaultBrowsePath = FPaths::ProjectLogDir();
		bSaved = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("StatsSaveTitle", "Save EQS stats").ToString(),
			DefaultBrowsePath,
			TEXT(""),
			FEnvironmentQueryHelper::SaveFileTypes,
			EFileDialogFlags::None,
			SaveFilenames
			);
	}

#if USE_EQS_DEBUGGER
	if (bSaved && SaveFilenames.Num() > 0 && SaveFilenames[0].IsEmpty() == false)
	{
		FEQSDebugger::SaveStats(SaveFilenames[0]);
	}
#endif
}

void FEnvironmentQueryEditor::OnLoadStats()
{
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	if (DesktopPlatform)
	{
		const FString DefaultBrowsePath = FPaths::ProjectLogDir();
		bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("StatsLoadTitle", "Load EQS stats").ToString(),
			DefaultBrowsePath,
			TEXT(""),
			FEnvironmentQueryHelper::LoadFileTypes,
			EFileDialogFlags::None,
			OpenFilenames
			);
	}

#if USE_EQS_DEBUGGER
	if (bOpened && OpenFilenames.Num() > 0 && OpenFilenames[0].IsEmpty() == false)
	{
		FEQSDebugger::LoadStats(OpenFilenames[0]);

		if (ProfilerView.IsValid())
		{
			ProfilerView->ForceUpdate();
		}
	}
#endif
}

void FEnvironmentQueryEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		FGraphPanelSelectionSet CurrentSelection = GetSelectedNodes();
		if (CurrentSelection.Num() == 1)
		{
			for (FGraphPanelSelectionSet::TConstIterator It(CurrentSelection); It; ++It)
			{
				UEnvironmentQueryGraphNode_Test* TestNode = Cast<UEnvironmentQueryGraphNode_Test>(*It);
				UEnvironmentQueryGraphNode_Option* ParentNode = TestNode ? Cast<UEnvironmentQueryGraphNode_Option>(TestNode->ParentNode) : nullptr;

				if (ParentNode)
				{
					ParentNode->CalculateWeights();
					break;
				}
			}
		}
	}
}

void FEnvironmentQueryEditor::OnStatsDataChange()
{
	TSharedPtr<SGraphEditor> UpdateGraphEditor = UpdateGraphEdPtr.Pin();
	UEnvironmentQueryGraph* EdGraph = UpdateGraphEditor.IsValid() ? Cast<UEnvironmentQueryGraph>(UpdateGraphEditor->GetCurrentGraph()) : nullptr;
	if (EdGraph)
	{
		// reset stats overlay
		EdGraph->ResetProfilerStats();

#if USE_EQS_DEBUGGER
		const bool bShowOverlay = ProfilerView.IsValid() && (ProfilerView->GetShowDetailsState() == ECheckBoxState::Checked);
		const FEQSDebugger::FStatsInfo* StatsInfo = bShowOverlay ? UEnvQueryManager::DebuggerStats.Find(ProfilerView->GetCurrentQueryKey()) : nullptr;
		
		if (StatsInfo && StatsInfo->TotalAvgCount)
		{
			EdGraph->StoreProfilerStats(*StatsInfo);
		}
#endif // USE_EQS_DEBUGGER
	}
}

#undef LOCTEXT_NAMESPACE
