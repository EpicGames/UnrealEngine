// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IEnvironmentQueryEditor.h"
#include "AIGraphEditor.h"

class FSpawnTabArgs;

class IDetailsView;
class SEnvQueryProfiler;
class UEdGraph;
class UEnvQuery;

class FEnvironmentQueryEditor : public IEnvironmentQueryEditor, public FAIGraphEditor
{

public:

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	void InitEnvironmentQueryEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UEnvQuery* Script );

	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Interface

	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);
	void OnStatsDataChange();

protected:
	/** Called when "Save" is clicked for this asset */
	virtual void SaveAsset_Execute() override;
	void ExtendToolbar();
	void BindCommands();

	void OnLoadStats();
	void OnSaveStats();

private:
	/** Create widget for graph editing */
	TSharedRef<class SGraphEditor> CreateGraphEditorWidget(UEdGraph* InGraph);

	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets();

	/** Called when the selection changes in the GraphEditor */
	void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection) override;

	/** Spawns the tab with the update graph inside */
	TSharedRef<SDockTab> SpawnTab_UpdateGraph(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Profiler(const FSpawnTabArgs& Args);

private:

	/* Query being edited */
	UEnvQuery* Query;

	/** Property View */
	TSharedPtr<class IDetailsView> DetailsView;
	TSharedPtr<SEnvQueryProfiler> ProfilerView;

	/**	Graph editor tab */
	static const FName EQSUpdateGraphTabId;
	static const FName EQSPropertiesTabId;
	static const FName EQSProfilerTabId;
};
