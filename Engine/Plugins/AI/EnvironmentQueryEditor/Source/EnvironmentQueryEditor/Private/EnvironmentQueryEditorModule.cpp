// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQueryEditorModule.h"

#include "Modules/ModuleManager.h"
#include "EnvironmentQueryGraphNode.h"
#include "PropertyEditorModule.h"
#include "EnvironmentQueryEditor.h"

#include "DetailCustomizations/EnvDirectionCustomization.h"
#include "DetailCustomizations/EnvTraceDataCustomization.h"
#include "DetailCustomizations/EnvQueryTestDetails.h"

#include "SGraphNode_EnvironmentQuery.h"
#include "EdGraphUtilities.h"

#include "EnvironmentQuery/Generators/EnvQueryGenerator_BlueprintBase.h"
 
IMPLEMENT_MODULE( FEnvironmentQueryEditorModule, EnvironmentQueryEditor );
DEFINE_LOG_CATEGORY(LogEnvironmentQueryEditor);

const FName FEnvironmentQueryEditorModule::EnvironmentQueryEditorAppIdentifier( TEXT( "EnvironmentQueryEditorApp" ) );

class FGraphPanelNodeFactory_EnvironmentQuery : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<class SGraphNode> CreateNode(UEdGraphNode* Node) const override
	{
		if (UEnvironmentQueryGraphNode* EnvQueryNode = Cast<UEnvironmentQueryGraphNode>(Node))
		{
			return SNew(SGraphNode_EnvironmentQuery, EnvQueryNode);
		}

		return NULL;
	}
};

TSharedPtr<FGraphPanelNodeFactory> GraphPanelNodeFactory_EnvironmentQuery;

void FEnvironmentQueryEditorModule::StartupModule()
{
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	GraphPanelNodeFactory_EnvironmentQuery = MakeShareable( new FGraphPanelNodeFactory_EnvironmentQuery() );
	FEdGraphUtilities::RegisterVisualNodeFactory(GraphPanelNodeFactory_EnvironmentQuery);

	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout( "EnvDirection", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FEnvDirectionCustomization::MakeInstance ) );
	PropertyModule.RegisterCustomPropertyTypeLayout( "EnvTraceData", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FEnvTraceDataCustomization::MakeInstance ) );
	PropertyModule.RegisterCustomClassLayout( "EnvQueryTest", FOnGetDetailCustomizationInstance::CreateStatic( &FEnvQueryTestDetails::MakeInstance ) );
	PropertyModule.NotifyCustomizationModuleChanged();
}


void FEnvironmentQueryEditorModule::ShutdownModule()
{
	if (!UObjectInitialized())
	{
		return;
	}

	ClassCache.Reset();
	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();

	if ( GraphPanelNodeFactory_EnvironmentQuery.IsValid() )
	{
		FEdGraphUtilities::UnregisterVisualNodeFactory(GraphPanelNodeFactory_EnvironmentQuery);
		GraphPanelNodeFactory_EnvironmentQuery.Reset();
	}

	// Unregister the details customization
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout( "EnvDirection" );
		PropertyModule.UnregisterCustomPropertyTypeLayout( "EnvTraceData" );

		PropertyModule.UnregisterCustomClassLayout( "EnvQueryTest" );
		PropertyModule.NotifyCustomizationModuleChanged();
	}
}


TSharedRef<IEnvironmentQueryEditor> FEnvironmentQueryEditorModule::CreateEnvironmentQueryEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UEnvQuery* Query)
{
	if (!ClassCache.IsValid())
	{
		ClassCache = MakeShareable(new FGraphNodeClassHelper(UEnvQueryNode::StaticClass()));
		FGraphNodeClassHelper::AddObservedBlueprintClasses(UEnvQueryGenerator_BlueprintBase::StaticClass());
		ClassCache->UpdateAvailableBlueprintClasses();
	}

	TSharedRef< FEnvironmentQueryEditor > NewEnvironmentQueryEditor( new FEnvironmentQueryEditor() );
	NewEnvironmentQueryEditor->InitEnvironmentQueryEditor(Mode, InitToolkitHost, Query);
	return NewEnvironmentQueryEditor;
}
