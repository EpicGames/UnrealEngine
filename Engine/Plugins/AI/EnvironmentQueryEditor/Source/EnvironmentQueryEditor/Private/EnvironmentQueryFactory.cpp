// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQueryFactory.h"
#include "EnvironmentQuery/EnvQuery.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvironmentQueryFactory)

UEnvironmentQueryFactory::UEnvironmentQueryFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UEnvQuery::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
}

UObject* UEnvironmentQueryFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UEnvQuery::StaticClass()));
	return NewObject<UEnvQuery>(InParent, Class, Name, Flags);
}

bool UEnvironmentQueryFactory::CanCreateNew() const
{
	// Always true if this plugin has been loaded
	return true;
}

