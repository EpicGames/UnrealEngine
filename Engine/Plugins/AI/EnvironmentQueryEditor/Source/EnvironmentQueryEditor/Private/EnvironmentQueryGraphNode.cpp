// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQueryGraphNode.h"
#include "EdGraphSchema_EnvironmentQuery.h"
#include "EnvironmentQueryGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvironmentQueryGraphNode)

UEnvironmentQueryGraphNode::UEnvironmentQueryGraphNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UEnvironmentQueryGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::GetEmpty();
}

FText UEnvironmentQueryGraphNode::GetDescription() const
{
	return FText::GetEmpty();
}

UEnvironmentQueryGraph* UEnvironmentQueryGraphNode::GetEnvironmentQueryGraph()
{
	return CastChecked<UEnvironmentQueryGraph>(GetGraph());
}

bool UEnvironmentQueryGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const
{
	return DesiredSchema->GetClass()->IsChildOf(UEdGraphSchema_EnvironmentQuery::StaticClass());
}

