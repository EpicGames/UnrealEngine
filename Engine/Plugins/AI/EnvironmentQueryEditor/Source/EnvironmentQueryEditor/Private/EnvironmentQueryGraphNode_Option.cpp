// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQueryGraphNode_Option.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvironmentQuery/EnvQueryOption.h"
#include "EnvironmentQueryGraphNode_Test.h"
#include "EdGraph/EdGraph.h"
#include "SGraphEditorActionMenuAI.h"
#include "EnvironmentQuery/Generators/EnvQueryGenerator_Composite.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvironmentQueryGraphNode_Option)

#define LOCTEXT_NAMESPACE "EnvironmentQueryEditor"

UEnvironmentQueryGraphNode_Option::UEnvironmentQueryGraphNode_Option(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bStatShowOverlay = false;
}

void UEnvironmentQueryGraphNode_Option::AllocateDefaultPins()
{
	UEdGraphPin* Inputs = CreatePin(EGPD_Input, TEXT("Transition"), TEXT("Out"));
}

void UEnvironmentQueryGraphNode_Option::PostPlacedNewNode()
{
	UClass* NodeClass = ClassData.GetClass(true);
	if (NodeClass)
	{
		UEdGraph* MyGraph = GetGraph();
		UObject* GraphOwner = MyGraph ? MyGraph->GetOuter() : nullptr;
		if (GraphOwner)
		{
			UEnvQueryOption* QueryOption = NewObject<UEnvQueryOption>(GraphOwner);
			QueryOption->Generator = NewObject<UEnvQueryGenerator>(GraphOwner, NodeClass);
			QueryOption->Generator->UpdateNodeVersion();

			QueryOption->SetFlags(RF_Transactional);
			QueryOption->Generator->SetFlags(RF_Transactional);

			NodeInstance = QueryOption;
			InitializeInstance();
		}
	}
}

void UEnvironmentQueryGraphNode_Option::ResetNodeOwner()
{
	Super::ResetNodeOwner();

	UEnvQueryOption* OptionInstance = Cast<UEnvQueryOption>(NodeInstance);
	if (OptionInstance && OptionInstance->Generator)
	{
		UObject* GraphOwner = GetGraph() ? GetGraph()->GetOuter() : nullptr;
		OptionInstance->Generator->Rename(NULL, GraphOwner, REN_DontCreateRedirectors | REN_DoNotDirty);
		OptionInstance->Generator->ClearFlags(RF_Transient);
	}
}

void UEnvironmentQueryGraphNode_Option::PrepareForCopying()
{
	Super::PrepareForCopying();

	UEnvQueryOption* OptionInstance = Cast<UEnvQueryOption>(NodeInstance);
	if (OptionInstance && OptionInstance->Generator)
	{
		// Temporarily take ownership of the node instance, so that it is not deleted when cutting
		OptionInstance->Generator->Rename(NULL, this, REN_DontCreateRedirectors | REN_DoNotDirty );
	}
}

void UEnvironmentQueryGraphNode_Option::UpdateNodeClassData()
{
	UEnvQueryOption* OptionInstance = Cast<UEnvQueryOption>(NodeInstance);
	if (OptionInstance && OptionInstance->Generator)
	{
		UpdateNodeClassDataFrom(OptionInstance->Generator->GetClass(), ClassData);
		ErrorMessage = ClassData.GetDeprecatedMessage();
	}
}

FText UEnvironmentQueryGraphNode_Option::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEnvQueryOption* OptionInstance = Cast<UEnvQueryOption>(NodeInstance);
	return OptionInstance ? OptionInstance->GetDescriptionTitle() : FText::GetEmpty();
}

FText UEnvironmentQueryGraphNode_Option::GetDescription() const
{
	UEnvQueryOption* OptionInstance = Cast<UEnvQueryOption>(NodeInstance);
	return OptionInstance ? OptionInstance->GetDescriptionDetails() : FText::GetEmpty();
}

void UEnvironmentQueryGraphNode_Option::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	FToolMenuSection& Section = Menu->AddSection("EnvironmentQueryGraphNode");
	Section.AddSubMenu(
		"AddTest",
		LOCTEXT("AddTest", "Add Test..." ),
		LOCTEXT("AddTestTooltip", "Adds new test to generator" ),
		FNewToolMenuDelegate::CreateUObject( this, &UEnvironmentQueryGraphNode_Option::CreateAddTestSubMenu,(UEdGraph*)Context->Graph) 
		);
}

void UEnvironmentQueryGraphNode_Option::CreateAddTestSubMenu(UToolMenu* Menu, UEdGraph* Graph) const
{
	TSharedRef<SGraphEditorActionMenuAI> Widget =	
		SNew(SGraphEditorActionMenuAI)
		.GraphObj( Graph )
		.GraphNode((UEnvironmentQueryGraphNode_Option*)this)
		.AutoExpandActionMenu(true);

	Menu->AddMenuEntry("Section", FToolMenuEntry::InitWidget("Widget", Widget, FText(), true));
}

void UEnvironmentQueryGraphNode_Option::CalculateWeights()
{
	float MaxWeight = -1.0f;
	for (int32 Idx = 0; Idx < SubNodes.Num(); Idx++)
	{
		UEnvironmentQueryGraphNode_Test* TestNode = Cast<UEnvironmentQueryGraphNode_Test>(SubNodes[Idx]);
		if (TestNode == nullptr || !TestNode->bTestEnabled)
		{
			continue;
		}

		UEnvQueryTest* TestInstance = Cast<UEnvQueryTest>(TestNode->NodeInstance);
		if (TestInstance && !TestInstance->ScoringFactor.IsDynamic())
		{
			MaxWeight = FMath::Max(MaxWeight, FMath::Abs(TestInstance->ScoringFactor.DefaultValue));
		}
	}

	if (MaxWeight <= 0.0f)
	{
		MaxWeight = 1.0f;
	}

	for (int32 Idx = 0; Idx < SubNodes.Num(); Idx++)
	{
		UEnvironmentQueryGraphNode_Test* TestNode = Cast<UEnvironmentQueryGraphNode_Test>(SubNodes[Idx]);
		if (TestNode == NULL)
		{
			continue;
		}
		
		UEnvQueryTest* TestInstance = Cast<UEnvQueryTest>(TestNode->NodeInstance);
		const bool bHasDynamic = TestInstance && TestNode->bTestEnabled && TestInstance->ScoringFactor.IsDynamic();
		const float NewWeight = (TestInstance && TestNode->bTestEnabled) ?
			(TestInstance->ScoringFactor.IsDynamic() ? 1.0f : FMath::Clamp(FMath::Abs(TestInstance->ScoringFactor.DefaultValue) / MaxWeight, 0.0f, 1.0f)) :
			-1.0f;

		TestNode->SetDisplayedWeight(NewWeight, bHasDynamic);
	}
}

void UEnvironmentQueryGraphNode_Option::UpdateNodeData()
{
	UEnvQueryOption* OptionInstance = Cast<UEnvQueryOption>(NodeInstance);
	UEnvQueryGenerator_Composite* CompositeGenerator = OptionInstance ? Cast<UEnvQueryGenerator_Composite>(OptionInstance->Generator) : nullptr;
	if (CompositeGenerator)
	{
		CompositeGenerator->VerifyItemTypes();

		ErrorMessage = CompositeGenerator->bHasMatchingItemType ? FString() : LOCTEXT("NestedGeneratorMismatch", "Nested generators must work on exactly the same item types!").ToString();
	}
}


#undef LOCTEXT_NAMESPACE

