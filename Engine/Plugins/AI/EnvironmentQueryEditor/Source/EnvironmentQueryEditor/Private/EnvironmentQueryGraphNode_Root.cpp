// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQueryGraphNode_Root.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvironmentQueryGraphNode_Root)

UEnvironmentQueryGraphNode_Root::UEnvironmentQueryGraphNode_Root(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bIsReadOnly = true;
}

void UEnvironmentQueryGraphNode_Root::AllocateDefaultPins()
{
	UEdGraphPin* Outputs = CreatePin(EGPD_Output, TEXT("Transition"), TEXT("In"));
}

FText UEnvironmentQueryGraphNode_Root::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("EnvironmentQueryEditor", "Root", "ROOT");
}

void UEnvironmentQueryGraphNode_Root::LogDebugMessage(const FString& Message)
{
	if (DebugMessages.Num() == 0)
	{
		bHasDebugError = false;
	}

	// store only 1 error message, discard everything after it
	if (!bHasDebugError)
	{
		DebugMessages.Add(Message);
	}
}

void UEnvironmentQueryGraphNode_Root::LogDebugError(const FString& Message)
{
	if (DebugMessages.Num() == 0)
	{
		bHasDebugError = false;
	}

	// store only 1 error message, discard everything after it
	if (!bHasDebugError)
	{
		DebugMessages.Add(Message);
		bHasDebugError = true;
	}
}

