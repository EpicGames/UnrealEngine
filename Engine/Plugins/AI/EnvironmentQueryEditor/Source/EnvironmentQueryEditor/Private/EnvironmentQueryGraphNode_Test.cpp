// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQueryGraphNode_Test.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvironmentQueryGraphNode_Option.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvironmentQueryGraphNode_Test)

UEnvironmentQueryGraphNode_Test::UEnvironmentQueryGraphNode_Test(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bIsSubNode = true;
	bTestEnabled = true;
	bHasNamedWeight = false;
	bStatShowOverlay = false;
	TestWeightPct = -1.0f;
}

void UEnvironmentQueryGraphNode_Test::InitializeInstance()
{
	UEnvQueryTest* TestInstance = Cast<UEnvQueryTest>(NodeInstance);
	if (TestInstance)
	{
		TestInstance->UpdateNodeVersion();
	}

	UEnvironmentQueryGraphNode_Option* ParentOption = Cast<UEnvironmentQueryGraphNode_Option>(ParentNode);
	if (ParentOption)
	{
		ParentOption->CalculateWeights();
	}
}

FText UEnvironmentQueryGraphNode_Test::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEnvQueryTest* TestInstance = Cast<UEnvQueryTest>(NodeInstance);
	return TestInstance ? TestInstance->GetDescriptionTitle() : FText::GetEmpty();
}

FText UEnvironmentQueryGraphNode_Test::GetDescription() const
{
	UEnvQueryTest* TestInstance = Cast<UEnvQueryTest>(NodeInstance);
	return TestInstance ? TestInstance->GetDescriptionDetails() : FText::GetEmpty();
}

void UEnvironmentQueryGraphNode_Test::SetDisplayedWeight(float Pct, bool bNamed)
{
	if (TestWeightPct != Pct || bHasNamedWeight != bNamed)
	{
		Modify();
	}

	TestWeightPct = Pct;
	bHasNamedWeight = bNamed;
}

