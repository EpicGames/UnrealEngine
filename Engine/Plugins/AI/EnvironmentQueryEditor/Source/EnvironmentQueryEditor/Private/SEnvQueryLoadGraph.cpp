// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEnvQueryLoadGraph.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"

void SEnvQueryLoadGraph::Construct(const FArguments& InArgs)
{

}

FVector2D SEnvQueryLoadGraph::ComputeDesiredSize(float) const
{
	return FVector2D(128, 92);
}

int32 SEnvQueryLoadGraph::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Rendering info
	bool bEnabled = ShouldBeEnabled(bParentEnabled);
	ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	const FSlateBrush* TimelineAreaBrush = FAppStyle::GetBrush("Profiler.LineGraphArea");
	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteTexture");

	// Draw timeline background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform()),
		TimelineAreaBrush,
		DrawEffects,
		TimelineAreaBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);

	LayerId++;

#if USE_EQS_DEBUGGER
	const float PaddingH = 2.0f;
	const float PaddingV = 5.0f;

	const int32 NumSamples = (Stats.LastTickEntry >= Stats.FirstTickEntry) ? (Stats.LastTickEntry - Stats.FirstTickEntry) : 0;
	const float GraphWidth = AllottedGeometry.GetLocalSize().X - (PaddingH * 2.0f);
	const float GraphHeight = AllottedGeometry.GetLocalSize().Y - (PaddingV * 2.0f);

	TArray<FVector2D> LinePoints;
	LinePoints.AddZeroed(2);

	if (GraphWidth > NumSamples)
	{
		// all samples can be shown on graph, make a line for each entry
		for (int32 Idx = 0; Idx < NumSamples; Idx++)
		{
			const float Pct = FMath::Min(1.0f, Stats.TickPct[Stats.FirstTickEntry + Idx] / 255.0f);
			const FLinearColor LineColor = (Pct < 0.3f) ? FLinearColor::White : (Pct < 0.6f) ? FLinearColor::Yellow : FLinearColor::Red;

			LinePoints[0].X = LinePoints[1].X = PaddingH + Idx;
			LinePoints[0].Y = AllottedGeometry.GetLocalSize().Y - PaddingV;
			LinePoints[1].Y = PaddingV + (1.0f - Pct) * GraphHeight;

			FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, DrawEffects, LineColor);
		}
	}
	else
	{
		// more samples than pixels, compress data
		const int32 NumSamplesPerLine = FMath::CeilToInt(NumSamples / GraphWidth);
		const int32 NumLines = NumSamples / NumSamplesPerLine;
		int32 SampleIdx = Stats.FirstTickEntry;

		for (int32 LineIdx = 0; LineIdx < NumLines; LineIdx++)
		{
			float Pct = 0.0f;
			for (int32 Idx = 0; Idx < NumSamplesPerLine; Idx++)
			{
				const float SamplePct = FMath::Min(1.0f, (SampleIdx < Stats.TickPct.Num()) ? Stats.TickPct[SampleIdx] / 255.0f : 0.0f);
				Pct = FMath::Max(Pct, SamplePct);
				SampleIdx++;
			}

			const FLinearColor LineColor = (Pct < 0.3f) ? FLinearColor::White : (Pct < 0.6f) ? FLinearColor::Yellow : FLinearColor::Red;

			LinePoints[0].X = LinePoints[1].X = PaddingH + LineIdx;
			LinePoints[0].Y = AllottedGeometry.GetLocalSize().Y - PaddingV;
			LinePoints[1].Y = PaddingV + (1.0f - Pct) * GraphHeight;

			FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, DrawEffects, LineColor);
		}
	}

	LayerId++;
#endif

	return LayerId;
}
