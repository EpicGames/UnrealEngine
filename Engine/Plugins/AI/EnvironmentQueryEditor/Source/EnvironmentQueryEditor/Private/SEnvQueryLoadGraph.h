// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "Widgets/SLeafWidget.h"

class FPaintArgs;
class FSlateWindowElementList;

class SEnvQueryLoadGraph : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SEnvQueryLoadGraph)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

#if USE_EQS_DEBUGGER
	FEQSDebugger::FStatsInfo Stats;
#endif
};
