// Copyright Epic Games, Inc. All Rights Reserved.


#include "SEnvQueryProfiler.h"
#include "Editor.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Input/SButton.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SCheckBox.h"
#include "Editor/UnrealEdEngine.h"
#include "SEnvQueryLoadGraph.h"
#include "UnrealEdGlobals.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "EnvironmentQuery/EnvQuery.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "EnvironmentQueryEditor"

namespace FEnvQueryProfilerIds
{
	const FName ColName = TEXT("Name");
	const FName ColMax = TEXT("MaxTime");
	const FName ColAvg = TEXT("AvgTime");
	const FName ColLoad = TEXT("AvgLoad");
	const FName ColCount = TEXT("AvgCount");
}

void SEnvQueryProfiler::Construct(const FArguments& InArgs)
{
	check(InArgs._OwnerQueryName.IsValid());
	OwnerQueryName = InArgs._OwnerQueryName;
	OnDataChanged = InArgs._OnDataChanged;
	bShowDetails = false;
	MatchingQueryName = NAME_None;

	TSharedPtr<SVerticalBox> VBox;
	
	ChildSlot.Padding(5, 5, 5, 5)
	[
		SAssignNew(VBox, SVerticalBox)
	];

	VBox->AddSlot().AutoHeight().Padding(0, 2)
		[
			SNew(STextBlock)
			.Text(this, &SEnvQueryProfiler::GetHeaderDesc)
		];
	VBox->AddSlot().AutoHeight().Padding(0, 2, 0, 10)
		[
			SNew(SCheckBox)
			.IsChecked(this, &SEnvQueryProfiler::GetShowDetailsState)
			.OnCheckStateChanged(this, &SEnvQueryProfiler::OnShowDetailsChanged)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProfilerDetailsOverlay","Show details for current query"))
			]
		];

	GraphView = SNew(SEnvQueryLoadGraph).Visibility(this, &SEnvQueryProfiler::GetGraphViewVisibility);
	BuildStatData();

	VBox->AddSlot().VAlign(VAlign_Top)
		[
			SAssignNew(ListView, SListView< FEnvQueryProfilerStatDataPtr >)
			.ListItemsSource(&StatData)
			.OnGenerateRow(this, &SEnvQueryProfiler::OnGenerateRowForList)
			.OnMouseButtonDoubleClick(this, &SEnvQueryProfiler::OnItemDoubleClicked)
			.SelectionMode(ESelectionMode::None)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(FEnvQueryProfilerIds::ColName).DefaultLabel(LOCTEXT("ProfilerListColName", "Name"))
				+ SHeaderRow::Column(FEnvQueryProfilerIds::ColMax).DefaultLabel(LOCTEXT("ProfilerListColMax", "Max (ms)"))
					.DefaultTooltip(LOCTEXT("ProfilerListColMaxTooltip", "Max run time (ms)"))
					.FixedWidth(60).HAlignCell(HAlign_Right).HAlignHeader(HAlign_Center).VAlignCell(VAlign_Center)
				+ SHeaderRow::Column(FEnvQueryProfilerIds::ColAvg).DefaultLabel(LOCTEXT("ProfilerListColAvg", "Avg (ms)"))
					.DefaultTooltip(LOCTEXT("ProfilerListColAvgTooltip", "Average run time (ms)"))
					.FixedWidth(60).HAlignCell(HAlign_Right).HAlignHeader(HAlign_Center).VAlignCell(VAlign_Center)
				+ SHeaderRow::Column(FEnvQueryProfilerIds::ColLoad).DefaultLabel(LOCTEXT("ProfilerListColLoad", "Load"))
					.DefaultTooltip(LOCTEXT("ProfilerListColLoadTooltip", "Average load of EQS tick"))
					.FixedWidth(60).HAlignCell(HAlign_Right).HAlignHeader(HAlign_Center).VAlignCell(VAlign_Center)
				+ SHeaderRow::Column(FEnvQueryProfilerIds::ColCount).DefaultLabel(LOCTEXT("ProfilerListColCount", "Count"))
					.DefaultTooltip(LOCTEXT("ProfilerListColCountTooltip", "Number of execution records"))
					.FixedWidth(60).HAlignCell(HAlign_Right).HAlignHeader(HAlign_Center).VAlignCell(VAlign_Center)
			)
		];

	VBox->AddSlot().AutoHeight().Padding(2)
		[
			SNew(STextBlock).Text(this, &SEnvQueryProfiler::GetGraphViewTitle)
		];

	VBox->AddSlot().AutoHeight()
		[
			GraphView.ToSharedRef()
		];
}

void SEnvQueryProfiler::BuildStatData()
{
	StatData.Reset();

#if USE_EQS_DEBUGGER

	const FString MatchPattern = OwnerQueryName.ToString() + FString("_");
	
	CachedGraphDesc = LOCTEXT("ProfilerNoGraph", "Open query with recorded data to view tick load graph");
	MatchingQueryName = NAME_None;

	for (auto It : UEnvQueryManager::DebuggerStats)
	{
		const FEQSDebugger::FStatsInfo& StatRecord = It.Value;
		
		FStatData NewInfo;
		NewInfo.QueryName = It.Key;
		NewInfo.WorstTime = StatRecord.MostExpensiveDuration;
		NewInfo.AverageTime = (StatRecord.TotalAvgCount > 0) ? (StatRecord.TotalAvgDuration / StatRecord.TotalAvgCount) : 0.0f;
		NewInfo.NumRuns = FMath::Max(0, StatRecord.TotalAvgCount);
		NewInfo.bIsHighlighted = It.Key.ToString().StartsWith(MatchPattern);

		if (NewInfo.bIsHighlighted)
		{
			MatchingQueryName = It.Key;

			const int32 NumSamples = FMath::Max(0, StatRecord.LastTickEntry - StatRecord.FirstTickEntry + 1);
			CachedGraphDesc = FText::Format(LOCTEXT("ProfilerGraphTitle", "Load of EQS tick budget for query: {0} (last {1} frames)"),
				FText::FromName(It.Key), FText::AsNumber(NumSamples));

			if (GraphView.IsValid())
			{
				GraphView->Stats = StatRecord;
			}
		}

		NewInfo.AverageLoad = 0;
		if (StatRecord.FirstTickEntry <= StatRecord.LastTickEntry)
		{
			for (int32 Idx = StatRecord.FirstTickEntry; Idx <= StatRecord.LastTickEntry; Idx++)
			{
				NewInfo.AverageLoad += StatRecord.TickPct[Idx] / 255.0f;
			}

			NewInfo.AverageLoad /= (StatRecord.LastTickEntry - StatRecord.FirstTickEntry + 1);
		}

		StatData.Add(MakeShareable(new FStatData(NewInfo)));
	}

	StatData.Sort([](const FEnvQueryProfilerStatDataPtr& A, const FEnvQueryProfilerStatDataPtr& B) { return (A->WorstTime > B->WorstTime); });

	if (StatData.Num() == 0)
	{
		CachedHeaderDesc = LOCTEXT("ProfilerHeaderNoData", "No data recorded, waiting for Play/Simulate in Editor game");
	}
	else
	{
		CachedHeaderDesc = FText::Format(LOCTEXT("ProfilerHeader", "Number of recorded queries: {0}"), FText::AsNumber(StatData.Num()));
	}

	TimeToNextUpdate = 2.0f;
#endif
}

void SEnvQueryProfiler::ForceUpdate()
{
	BuildStatData();

	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}

	if (bShowDetails && (MatchingQueryName != NAME_None))
	{
		OnDataChanged.ExecuteIfBound();
	}
}

void SEnvQueryProfiler::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const bool bIsPlaySessionRunning = GUnrealEd->PlayWorld && !GUnrealEd->PlayWorld->bDebugPauseExecution;
	if (!bIsPlaySessionRunning)
	{
		return;
	}

	TimeToNextUpdate -= InDeltaTime;
	if (TimeToNextUpdate < 0.0f)
	{
		ForceUpdate();
	}
}

static UObject* FindQueryObjectByName(FName StatName)
{
	FString AssetPathName = StatName.ToString();
	int32 SepIdx = INDEX_NONE;
	if (AssetPathName.FindLastChar(TEXT('_'), SepIdx))
	{
		AssetPathName.LeftInline(SepIdx);
	}

	UObject* QueryOb = FindFirstObject<UObject>(*AssetPathName, EFindFirstObjectOptions::NativeFirst);
	if (QueryOb == nullptr)
	{
		TArray<FAssetData> Assets;
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.GetAssetsByClass(UEnvQuery::StaticClass()->GetClassPathName(), Assets);

		const FString MatchObjectName = FString(".") + AssetPathName;
		for (int32 Idx = 0; Idx < Assets.Num(); Idx++)
		{
			const FString AssetNameStr = Assets[Idx].GetObjectPathString();
			if (AssetNameStr.EndsWith(MatchObjectName))
			{
				QueryOb = Assets[Idx].GetAsset();
				break;
			}
		}
	}

	return QueryOb;
}

void SEnvQueryProfiler::OnItemDoubleClicked(TSharedPtr<SEnvQueryProfiler::FStatData, ESPMode::ThreadSafe> Item)
{
	UObject* QueryOb = FindQueryObjectByName(Item->QueryName);
	if (QueryOb)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(QueryOb);
	}
}

TSharedRef<ITableRow> SEnvQueryProfiler::OnGenerateRowForList(FEnvQueryProfilerStatDataPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SEnvQueryProfilerTableRow, OwnerTable, Item);
}

ECheckBoxState SEnvQueryProfiler::GetShowDetailsState() const
{
	return bShowDetails ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SEnvQueryProfiler::OnShowDetailsChanged(ECheckBoxState NewState)
{
	bShowDetails = (NewState == ECheckBoxState::Checked);
	OnDataChanged.ExecuteIfBound();
}

EVisibility SEnvQueryProfiler::GetGraphViewVisibility() const
{
	return (MatchingQueryName != NAME_None) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SEnvQueryProfilerTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const FEnvQueryProfilerStatDataPtr InStatInfo)
{
	StatInfo = InStatInfo;

	SMultiColumnTableRow<FEnvQueryProfilerStatDataPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	SetBorderImage(TAttribute<const FSlateBrush*>(this, &SEnvQueryProfilerTableRow::GetBorder));
}

const FSlateBrush* SEnvQueryProfilerTableRow::GetBorder() const
{
	if (StatInfo->bIsHighlighted)
	{
		return &Style->InactiveBrush;
	}

	return STableRow<FEnvQueryProfilerStatDataPtr>::GetBorder();
}

TSharedRef<SWidget> SEnvQueryProfilerTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == FEnvQueryProfilerIds::ColName)
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.Text(NSLOCTEXT("PropertyCustomizationHelpers", "BrowseButtonLabel", "Browse"))
				.ToolTipText(LOCTEXT("BrowseButtonToolTipText", "Browse to Asset in Content Browser"))
				.OnClicked(this, &SEnvQueryProfilerTableRow::OnBrowseClicked)
				.ContentPadding(4.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Search"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(FText::FromName(StatInfo->QueryName))
			];
	}
	else if (ColumnName == FEnvQueryProfilerIds::ColMax)
	{
		FNumberFormattingOptions FmtOptions;
		FmtOptions.SetMaximumFractionalDigits(2);

		return SNew(STextBlock).Text(FText::AsNumber(StatInfo->WorstTime * 1000.0f, &FmtOptions));
	}
	else if (ColumnName == FEnvQueryProfilerIds::ColAvg)
	{
		FNumberFormattingOptions FmtOptions;
		FmtOptions.SetMaximumFractionalDigits(2);

		return SNew(STextBlock).Text(FText::AsNumber(StatInfo->AverageTime * 1000.0f, &FmtOptions));
	}
	else if (ColumnName == FEnvQueryProfilerIds::ColLoad)
	{
		return SNew(STextBlock).Text(FText::AsPercent(StatInfo->AverageLoad));
	}
	else if (ColumnName == FEnvQueryProfilerIds::ColCount)
	{
		return SNew(STextBlock).Text(FText::AsNumber(StatInfo->NumRuns));
	}

	return SNullWidget::NullWidget;
}

FReply SEnvQueryProfilerTableRow::OnBrowseClicked()
{
	UObject* QueryOb = FindQueryObjectByName(StatInfo->QueryName);
	if (QueryOb)
	{
		TArray<UObject*> Objects;
		Objects.Add(QueryOb);
		GEditor->SyncBrowserToObjects(Objects);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
