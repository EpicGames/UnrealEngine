// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h" // IWYU pragma: keep

class SEnvQueryLoadGraph;
class STableViewBase;

class SEnvQueryProfiler : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEnvQueryProfiler) {}
		SLATE_ARGUMENT(FName, OwnerQueryName)
		SLATE_EVENT(FSimpleDelegate, OnDataChanged)
	SLATE_END_ARGS()

	struct FStatData : public TSharedFromThis<FStatData, ESPMode::ThreadSafe>
	{
		FName QueryName;
		double WorstTime;
		double AverageTime;
		float AverageLoad;
		int32 NumRuns;
		bool bIsHighlighted;
	};

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void Construct(const FArguments& InArgs);
	void BuildStatData();
	void ForceUpdate();

	TSharedRef<ITableRow> OnGenerateRowForList(TSharedPtr<FStatData, ESPMode::ThreadSafe> Item, const TSharedRef<STableViewBase>& OwnerTable);
	FText GetHeaderDesc() const { return CachedHeaderDesc; }
	FText GetGraphViewTitle() const { return CachedGraphDesc; }
	EVisibility GetGraphViewVisibility() const;
	ECheckBoxState GetShowDetailsState() const;
	void OnShowDetailsChanged(ECheckBoxState NewState);
	void OnItemDoubleClicked(TSharedPtr<SEnvQueryProfiler::FStatData, ESPMode::ThreadSafe> Item);

	FName GetCurrentQueryKey() const { return MatchingQueryName; }

private:

	TArray<TSharedPtr<FStatData, ESPMode::ThreadSafe> > StatData;
	TSharedPtr<SListView< TSharedPtr<FStatData, ESPMode::ThreadSafe> > > ListView;
	TSharedPtr<SEnvQueryLoadGraph> GraphView;

	FText CachedHeaderDesc;
	FText CachedGraphDesc;
	FName OwnerQueryName;
	FName MatchingQueryName;
	FSimpleDelegate OnDataChanged;

	float TimeToNextUpdate;
	uint32 bShowDetails : 1;
};

typedef TSharedPtr<SEnvQueryProfiler::FStatData, ESPMode::ThreadSafe> FEnvQueryProfilerStatDataPtr;

class SEnvQueryProfilerTableRow : public SMultiColumnTableRow<FEnvQueryProfilerStatDataPtr>
{
public:
	SLATE_BEGIN_ARGS(SEnvQueryProfilerTableRow) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const FEnvQueryProfilerStatDataPtr InStatInfo);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	const FSlateBrush* GetBorder() const;
	FReply OnBrowseClicked();

private:

	FEnvQueryProfilerStatDataPtr StatInfo;
};
