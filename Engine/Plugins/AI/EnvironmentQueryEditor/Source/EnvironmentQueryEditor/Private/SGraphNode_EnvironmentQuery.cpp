// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphNode_EnvironmentQuery.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvironmentQuery/EnvQueryOption.h"
#include "EnvironmentQueryGraph.h"
#include "EnvironmentQueryGraphNode_Option.h"
#include "EnvironmentQueryGraphNode_Test.h"
#include "GraphEditorSettings.h"
#include "SCommentBubble.h"
#include "NodeFactory.h"
#include "SGraphPanel.h"
#include "EnvironmentQueryColors.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "SLevelOfDetailBranchNode.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "EnvironmentQueryEditor"

class SEnvironmentQueryPin : public SGraphPinAI
{
public:
	SLATE_BEGIN_ARGS(SEnvironmentQueryPin){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

	virtual FSlateColor GetPinColor() const override;
};

void SEnvironmentQueryPin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPinAI::Construct(SGraphPinAI::FArguments(), InPin);
}

FSlateColor SEnvironmentQueryPin::GetPinColor() const
{
	return IsHovered() ? EnvironmentQueryColors::Pin::Hover :
		EnvironmentQueryColors::Pin::Default;
}

/////////////////////////////////////////////////////
// SGraphNode_EnvironmentQuery

void SGraphNode_EnvironmentQuery::Construct(const FArguments& InArgs, UEnvironmentQueryGraphNode* InNode)
{
	SGraphNodeAI::Construct(SGraphNodeAI::FArguments(), InNode);
}

void SGraphNode_EnvironmentQuery::AddSubNode(TSharedPtr<SGraphNode> SubNodeWidget)
{
	SGraphNodeAI::AddSubNode(SubNodeWidget);

	TestBox->AddSlot().AutoHeight()
		[
			SubNodeWidget.ToSharedRef()
		];
}

FSlateColor SGraphNode_EnvironmentQuery::GetBorderBackgroundColor() const
{
	UEnvironmentQueryGraphNode_Test* TestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	const bool bSelectedSubNode = TestNode && TestNode->ParentNode && GetOwnerPanel().IsValid() && GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode);

	return bSelectedSubNode ? EnvironmentQueryColors::NodeBorder::Selected :
		EnvironmentQueryColors::NodeBorder::Default;
}

FSlateColor SGraphNode_EnvironmentQuery::GetBackgroundColor() const
{
	const UEnvironmentQueryGraphNode* MyNode = Cast<UEnvironmentQueryGraphNode>(GraphNode);
	const UClass* MyClass = MyNode && MyNode->NodeInstance ? MyNode->NodeInstance->GetClass() : NULL;
	
	FLinearColor NodeColor = EnvironmentQueryColors::NodeBody::Default;
	if (MyClass)
	{
		if (MyClass->IsChildOf( UEnvQueryTest::StaticClass() ))
		{
			UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
			NodeColor = (MyTestNode && MyTestNode->bTestEnabled) ? EnvironmentQueryColors::NodeBody::Test : EnvironmentQueryColors::NodeBody::TestInactive;
		}
		else if (MyClass->IsChildOf( UEnvQueryOption::StaticClass() ))
		{
			NodeColor = EnvironmentQueryColors::NodeBody::Generator;
		}
	}

	if (!MyNode || MyNode->HasErrors())
	{
		NodeColor = EnvironmentQueryColors::NodeBody::Error;
	}

	return NodeColor;
}

void SGraphNode_EnvironmentQuery::UpdateGraphNode()
{
	if (TestBox.IsValid())
	{
		TestBox->ClearChildren();
	} 
	else
	{
		SAssignNew(TestBox,SVerticalBox);
	}

	InputPins.Empty();
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();
	SubNodes.Reset();

	const FMargin NodePadding = (Cast<UEnvironmentQueryGraphNode_Test>(GraphNode) != NULL) 
		? FMargin(2.0f) 
		: FMargin(8.0f);
	float TestPadding = 0.0f;

	UEnvironmentQueryGraphNode_Option* OptionNode = Cast<UEnvironmentQueryGraphNode_Option>(GraphNode);
	if (OptionNode)
	{
		for (int32 Idx = 0; Idx < OptionNode->SubNodes.Num(); Idx++)
		{
			if (OptionNode->SubNodes[Idx])
			{
				TSharedPtr<SGraphNode> NewNode = FNodeFactory::CreateNodeWidget(OptionNode->SubNodes[Idx]);
				if (OwnerGraphPanelPtr.IsValid())
				{
					NewNode->SetOwner(OwnerGraphPanelPtr.Pin().ToSharedRef());
					OwnerGraphPanelPtr.Pin()->AttachGraphEvents(NewNode);
				}
				AddSubNode(NewNode);
				NewNode->UpdateGraphNode();
			}
		}

		if (SubNodes.Num() == 0)
		{
			TestBox->AddSlot().AutoHeight()
				[
					SNew(STextBlock).Text(LOCTEXT("NoTests","Right click to add tests"))
				];
		}

		TestPadding = 10.0f;
	}

	const FSlateBrush* NodeTypeIcon = GetNameIcon();

	FLinearColor TitleShadowColor(1.0f, 0.0f, 0.0f);
	TSharedPtr<SErrorText> ErrorText;
	TSharedPtr<STextBlock> DescriptionText; 
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	TWeakPtr<SNodeTitle> WeakNodeTitle = NodeTitle;
	auto GetNodeTitlePlaceholderWidth = [WeakNodeTitle]() -> FOptionalSize
	{
		TSharedPtr<SNodeTitle> NodeTitlePin = WeakNodeTitle.Pin();
		const float DesiredWidth = (NodeTitlePin.IsValid()) ? NodeTitlePin->GetTitleSize().X : 0.0f;
		return FMath::Max(75.0f, DesiredWidth);
	};
	auto GetNodeTitlePlaceholderHeight = [WeakNodeTitle]() -> FOptionalSize
	{
		TSharedPtr<SNodeTitle> NodeTitlePin = WeakNodeTitle.Pin();
		const float DesiredHeight = (NodeTitlePin.IsValid()) ? NodeTitlePin->GetTitleSize().Y : 0.0f;
		return FMath::Max(22.0f, DesiredHeight);
	};

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush( "Graph.StateNode.Body" ) )
			.Padding(0)
			.BorderBackgroundColor( this, &SGraphNode_EnvironmentQuery::GetBorderBackgroundColor )
			.OnMouseButtonDown(this, &SGraphNode_EnvironmentQuery::OnMouseDown)
			[
				SNew(SOverlay)

				// Pins and node details
				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SVerticalBox)

					// INPUT PIN AREA
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.MinDesiredHeight(NodePadding.Top)
						[
							SAssignNew(LeftNodeBox, SVerticalBox)
						]
					]

					// STATE NAME AREA
					+SVerticalBox::Slot()
					.Padding(FMargin(NodePadding.Left, 0.0f, NodePadding.Right, 0.0f))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(SCheckBox)
							.Visibility(this, &SGraphNode_EnvironmentQuery::GetTestToggleVisibility)
							.IsChecked(this, &SGraphNode_EnvironmentQuery::IsTestToggleChecked)
							.OnCheckStateChanged(this, &SGraphNode_EnvironmentQuery::OnTestToggleChanged)
							.Padding(FMargin(0, 0, 4.0f, 0))
						]
						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SBorder)
								.BorderImage( FAppStyle::GetBrush("Graph.StateNode.Body") )
								.BorderBackgroundColor( this, &SGraphNode_EnvironmentQuery::GetBackgroundColor )
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Center)
								.Visibility(EVisibility::SelfHitTestInvisible)
								[
									SNew(SVerticalBox)
									+SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0,0,0,2)
									[
										SNew(SBox).HeightOverride(4.f)
										[
											// weight bar
											SNew(SProgressBar)
											.FillColorAndOpacity(this, &SGraphNode_EnvironmentQuery::GetWeightProgressBarColor)
											.Visibility(this, &SGraphNode_EnvironmentQuery::GetWeightMarkerVisibility)
											.Percent(this, &SGraphNode_EnvironmentQuery::GetWeightProgressBarPercent)
										]
									]
									+SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)
										+SHorizontalBox::Slot()
										.AutoWidth()
										[
											// POPUP ERROR MESSAGE
											SAssignNew(ErrorText, SErrorText )
											.BackgroundColor( this, &SGraphNode_EnvironmentQuery::GetErrorColor )
											.ToolTipText( this, &SGraphNode_EnvironmentQuery::GetErrorMsgToolTip )
										]
										+SHorizontalBox::Slot()
										.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
										[
											SNew(SLevelOfDetailBranchNode)
											.UseLowDetailSlot(this, &SGraphNode_EnvironmentQuery::UseLowDetailNodeTitles)
											.LowDetail()
											[
												SNew(SBox)
												.WidthOverride_Lambda(GetNodeTitlePlaceholderWidth)
												.HeightOverride_Lambda(GetNodeTitlePlaceholderHeight)
											]
											.HighDetail()
											[
												SNew(SVerticalBox)
												+SVerticalBox::Slot()
												.AutoHeight()
												[
													SAssignNew(InlineEditableText, SInlineEditableTextBlock)
													.Style(FAppStyle::Get(), "Graph.StateNode.NodeTitleInlineEditableText")
													.Text(this, &SGraphNode_EnvironmentQuery::GetTitle)
													.OnVerifyTextChanged(this, &SGraphNode_EnvironmentQuery::OnVerifyNameTextChanged)
													.OnTextCommitted(this, &SGraphNode_EnvironmentQuery::OnNameTextCommited)
													.IsReadOnly(this, &SGraphNode_EnvironmentQuery::IsNameReadOnly )
													.IsSelected(this, &SGraphNode_EnvironmentQuery::IsSelectedExclusively)
												]
												+SVerticalBox::Slot()
												.AutoHeight()
												[
													NodeTitle.ToSharedRef()
												]
											]
										]
									]
									+SVerticalBox::Slot()
									.AutoHeight()
									[
										// DESCRIPTION MESSAGE
										SAssignNew(DescriptionText, STextBlock )
										.Visibility(this, &SGraphNode_EnvironmentQuery::GetDescriptionVisibility)
										.Text(this, &SGraphNode_EnvironmentQuery::GetDescription)
									]
								]
							]
							+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0, TestPadding, 0, 0)
							[
								TestBox.ToSharedRef()
							]
						]
					]

					// OUTPUT PIN AREA
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.MinDesiredHeight(NodePadding.Bottom)
						[
							SAssignNew(RightNodeBox, SVerticalBox)
						]
					]

					// Profiler overlay: option
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.BorderBackgroundColor(EnvironmentQueryColors::Action::Profiler)
						.Visibility(this, &SGraphNode_EnvironmentQuery::GetProfilerOptionVisibility)
						[
							SNew(STextBlock)
							.Text(this, &SGraphNode_EnvironmentQuery::GetProfilerDescOption)
						]
					]
				]

				// Drag marker overlay
				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					SNew(SBorder)
					.BorderBackgroundColor(EnvironmentQueryColors::Action::DragMarker)
					.ColorAndOpacity(EnvironmentQueryColors::Action::DragMarker)
					.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
					.Visibility(this, &SGraphNode_EnvironmentQuery::GetDragOverMarkerVisibility)
					[
						SNew(SBox)
						.HeightOverride(4)
					]
				]

				// Profiler overlay: test
				+SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Fill)
				.Padding(10, 5)
				[
					SNew(SBorder)
					.BorderBackgroundColor(EnvironmentQueryColors::Action::Profiler)
					.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
					.Visibility(this, &SGraphNode_EnvironmentQuery::GetProfilerTestVisibility)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
							.BorderBackgroundColor(this, &SGraphNode_EnvironmentQuery::GetProfilerTestSlateColor)
							[
								SNew(SBox)
								.WidthOverride(10.0f)
							]
						]
						+SHorizontalBox::Slot()
						.Padding(2, 0, 0, 0)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(this, &SGraphNode_EnvironmentQuery::GetProfilerDescAverage)
							]
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(this, &SGraphNode_EnvironmentQuery::GetProfilerDescWorst)
							]
						]
					]
				]
			]
		];

	// Create comment bubble
	TSharedPtr<SCommentBubble> CommentBubble;
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew(CommentBubble, SCommentBubble)
		.GraphNode(GraphNode)
		.Text(this, &SGraphNode::GetNodeComment)
		.OnTextCommitted( this, &SGraphNode::OnCommentTextCommitted )
		.ColorAndOpacity(CommentColor)
		.AllowPinning(true)
		.EnableTitleBarBubble(true)
		.EnableBubbleCtrls(true)
		.GraphLOD(this, &SGraphNode::GetCurrentLOD)
		.IsGraphNodeHovered(this, &SGraphNode::IsHovered);

	GetOrAddSlot(ENodeZone::TopCenter)
		.SlotOffset2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetOffset2f))
		.SlotSize2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetSize2f))
		.AllowScaling(TAttribute<bool>(CommentBubble.Get(), &SCommentBubble::IsScalingAllowed))
		.VAlign(VAlign_Top)
		[
			CommentBubble.ToSharedRef()
		];

	ErrorReporting = ErrorText;
	ErrorReporting->SetError(ErrorMsg);
	CreatePinWidgets();
}

void SGraphNode_EnvironmentQuery::CreatePinWidgets()
{
	UEnvironmentQueryGraphNode* StateNode = CastChecked<UEnvironmentQueryGraphNode>(GraphNode);

	UEdGraphPin* CurPin = StateNode->GetOutputPin();
	if (CurPin && !CurPin->bHidden)
	{
		TSharedPtr<SGraphPin> NewPin = SNew(SEnvironmentQueryPin, CurPin);

		AddPin(NewPin.ToSharedRef());
	}

	CurPin = StateNode->GetInputPin();
	if (CurPin && !CurPin->bHidden)
	{
		TSharedPtr<SGraphPin> NewPin = SNew(SEnvironmentQueryPin, CurPin);

		AddPin(NewPin.ToSharedRef());
	}
}

EVisibility SGraphNode_EnvironmentQuery::GetWeightMarkerVisibility() const
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	return MyTestNode ? EVisibility::Visible : EVisibility::Collapsed;
}

TOptional<float> SGraphNode_EnvironmentQuery::GetWeightProgressBarPercent() const
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	return MyTestNode ? FMath::Max(0.0f, MyTestNode->TestWeightPct) : TOptional<float>();
}

FSlateColor SGraphNode_EnvironmentQuery::GetWeightProgressBarColor() const
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	return (MyTestNode && MyTestNode->bHasNamedWeight) ? EnvironmentQueryColors::Action::WeightNamed : EnvironmentQueryColors::Action::Weight;
}

EVisibility SGraphNode_EnvironmentQuery::GetTestToggleVisibility() const
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	return MyTestNode ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState SGraphNode_EnvironmentQuery::IsTestToggleChecked() const
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	return MyTestNode && MyTestNode->bTestEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SGraphNode_EnvironmentQuery::OnTestToggleChanged(ECheckBoxState NewState)
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	if (MyTestNode)
	{
		MyTestNode->bTestEnabled = (NewState == ECheckBoxState::Checked);
		
		UEnvironmentQueryGraphNode_Option* MyParentNode = Cast<UEnvironmentQueryGraphNode_Option>(MyTestNode->ParentNode);
		if (MyParentNode)
		{
			MyParentNode->CalculateWeights();
		}

		UEnvironmentQueryGraph* MyGraph = Cast<UEnvironmentQueryGraph>(MyTestNode->GetGraph());
		if (MyGraph)
		{
			MyGraph->UpdateAsset();
		}
	}
}

FSlateColor SGraphNode_EnvironmentQuery::GetProfilerTestSlateColor() const
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	if (MyTestNode)
	{
		return MyTestNode->Stats.AvgTime >= 5 ? FLinearColor::Red :
			MyTestNode->Stats.AvgTime >= 2 ? FLinearColor::Yellow :
			FLinearColor::Green;
	}

	return FLinearColor::White;		
}

EVisibility SGraphNode_EnvironmentQuery::GetProfilerTestVisibility() const
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	return MyTestNode && MyTestNode->bStatShowOverlay ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SGraphNode_EnvironmentQuery::GetProfilerOptionVisibility() const
{
	UEnvironmentQueryGraphNode_Option* MyOptionNode = Cast<UEnvironmentQueryGraphNode_Option>(GraphNode);
	return MyOptionNode && MyOptionNode->bStatShowOverlay ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SGraphNode_EnvironmentQuery::GetProfilerDescAverage() const
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	if (MyTestNode && MyTestNode->bStatShowOverlay)
	{
		FNumberFormattingOptions FmtOptions;
		FmtOptions.SetMaximumFractionalDigits(2);

		return FText::Format(LOCTEXT("ProfilerOverlayAvg", "Average run: {0} ms"), FText::AsNumber(MyTestNode->Stats.AvgTime, &FmtOptions));
	}

	return FText::GetEmpty();
}

FText SGraphNode_EnvironmentQuery::GetProfilerDescWorst() const
{
	UEnvironmentQueryGraphNode_Test* MyTestNode = Cast<UEnvironmentQueryGraphNode_Test>(GraphNode);
	if (MyTestNode && MyTestNode->bStatShowOverlay)
	{
		FNumberFormattingOptions FmtOptions;
		FmtOptions.SetMaximumFractionalDigits(2);
		
		return FText::Format(LOCTEXT("ProfilerOverlayMax", "Worst run: {0} ms, {1} items"), FText::AsNumber(MyTestNode->Stats.MaxTime, &FmtOptions), FText::AsNumber(MyTestNode->Stats.MaxNumProcessedItems));
	}

	return FText::GetEmpty();
}

FText SGraphNode_EnvironmentQuery::GetProfilerDescOption() const
{
	UEnvironmentQueryGraphNode_Option* MyOptionNode = Cast<UEnvironmentQueryGraphNode_Option>(GraphNode);
	if (MyOptionNode && MyOptionNode->bStatShowOverlay)
	{
		FTextBuilder DescBuilder;

		FNumberFormattingOptions FmtOptions;
		FmtOptions.SetMaximumFractionalDigits(2);

		for (int32 Idx = 0; Idx < MyOptionNode->StatsPerGenerator.Num(); Idx++)
		{
			DescBuilder.AppendLineFormat(LOCTEXT("ProfilerOverlayGen", "Generator[{0}]"), Idx);
			DescBuilder.Indent();

			DescBuilder.AppendLineFormat(LOCTEXT("ProfilerOverlayAvg", "Average run: {0} ms"), FText::AsNumber(MyOptionNode->StatsPerGenerator[Idx].AvgTime, &FmtOptions));
			DescBuilder.AppendLineFormat(LOCTEXT("ProfilerOverlayMax", "Worst run: {0} ms, {1} items"), FText::AsNumber(MyOptionNode->StatsPerGenerator[Idx].MaxTime, &FmtOptions), FText::AsNumber(MyOptionNode->StatsPerGenerator[Idx].MaxNumProcessedItems));

			DescBuilder.Unindent();
		}

		DescBuilder.AppendLine();
		DescBuilder.AppendLineFormat(LOCTEXT("ProfilerOverlayPickRate", "Pick rate: {0}"), FText::AsPercent(MyOptionNode->StatAvgPickRate));
		return DescBuilder.ToText();
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
