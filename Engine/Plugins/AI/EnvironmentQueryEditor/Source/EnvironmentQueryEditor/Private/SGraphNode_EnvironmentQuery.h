// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNodeAI.h"

class SVerticalBox;
class UEnvironmentQueryGraphNode;

class SGraphNode_EnvironmentQuery : public SGraphNodeAI
{
public:
	SLATE_BEGIN_ARGS(SGraphNode_EnvironmentQuery){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEnvironmentQueryGraphNode* InNode);

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void CreatePinWidgets() override;
	// End of SGraphNode interface

	/** adds decorator widget inside current node */
	virtual void AddSubNode(TSharedPtr<SGraphNode> SubNodeWidget) override;

	EVisibility GetWeightMarkerVisibility() const;
	TOptional<float> GetWeightProgressBarPercent() const;
	FSlateColor GetWeightProgressBarColor() const; 

	EVisibility GetTestToggleVisibility() const;
	ECheckBoxState IsTestToggleChecked() const;
	void OnTestToggleChanged(ECheckBoxState NewState);

	FSlateColor GetProfilerTestSlateColor() const;
	EVisibility GetProfilerTestVisibility() const;
	EVisibility GetProfilerOptionVisibility() const;
	FText GetProfilerDescAverage() const;
	FText GetProfilerDescWorst() const;
	FText GetProfilerDescOption() const;

protected:
	TSharedPtr<SVerticalBox> TestBox;

	FSlateColor GetBorderBackgroundColor() const;
	FSlateColor GetBackgroundColor() const;
};
