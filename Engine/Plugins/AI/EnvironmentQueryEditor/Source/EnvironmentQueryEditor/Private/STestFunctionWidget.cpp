// Copyright Epic Games, Inc. All Rights Reserved.

#include "STestFunctionWidget.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"

void STestFunctionWidget::Construct(const FArguments& InArgs)
{

}

FVector2D STestFunctionWidget::GetWidgetPosition(float X, float Y, const FGeometry& Geom) const
{
	return FVector2D((X*Geom.GetLocalSize().X), (Geom.GetLocalSize().Y - 1) - (Y*Geom.GetLocalSize().Y));
}

int32 STestFunctionWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Rendering info
	bool bEnabled = ShouldBeEnabled(bParentEnabled);
	ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	const FSlateBrush* TimelineAreaBrush = FAppStyle::GetBrush("Profiler.LineGraphArea");
	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteTexture");

	// Draw timeline background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(AllottedGeometry.GetLocalSize(), FSlateLayoutTransform()),
		TimelineAreaBrush,
		DrawEffects,
		TimelineAreaBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);

	LayerId++;

	UEnvQueryTest* TestOb = DrawTestOb.Get();
	if (TestOb == nullptr)
	{
		return LayerId;
	}

	const FEnvQueryTestScoringPreview& PreviewData = TestOb->PreviewData;

	// Draw filter background
	if (PreviewData.bShowFilterLow)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2f(FMath::TruncToFloat(PreviewData.FilterLow * AllottedGeometry.Size.X), AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(0.f, 0.f))),
			WhiteBrush,
			DrawEffects,
			WhiteBrush->GetTint(InWidgetStyle) * FLinearColor(1.0f, 0.0f, 0.0f, 0.4f)
			);

		LayerId++;
	}

	if (PreviewData.bShowFilterHigh)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(AllottedGeometry.GetLocalSize(), FSlateLayoutTransform(FVector2f(FMath::TruncToFloat(PreviewData.FilterHigh * AllottedGeometry.Size.X), 0.f))),
			WhiteBrush,
			DrawEffects,
			WhiteBrush->GetTint(InWidgetStyle) * FLinearColor(1.0f, 0.0f, 0.0f, 0.5f)
			);

		LayerId++;
	}

	// Draw axies
	TArray<FVector2D> AxisPoints;
	AxisPoints.Add(GetWidgetPosition(0, 1, AllottedGeometry));
	AxisPoints.Add(GetWidgetPosition(0, 0, AllottedGeometry));
	AxisPoints.Add(GetWidgetPosition(1, 0, AllottedGeometry));

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		AxisPoints,
		DrawEffects,
		WhiteBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);

	LayerId++;

	// Draw clamping and filtering filters
	if (PreviewData.bShowClampMin)
	{
		TArray<FVector2D> ClampLine;
		ClampLine.Add(GetWidgetPosition(PreviewData.ClampMin, 0, AllottedGeometry));
		ClampLine.Add(GetWidgetPosition(PreviewData.ClampMin, 1, AllottedGeometry));

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			ClampLine,
			DrawEffects,
			WhiteBrush->GetTint(InWidgetStyle) * FLinearColor(1.0f, 1.0f, 0.0f, 1.0f)
			);

		LayerId++;
	}

	if (PreviewData.bShowClampMax)
	{
		TArray<FVector2D> ClampLine;
		ClampLine.Add(GetWidgetPosition(PreviewData.ClampMax, 0, AllottedGeometry));
		ClampLine.Add(GetWidgetPosition(PreviewData.ClampMax, 1, AllottedGeometry));

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			ClampLine,
			DrawEffects,
			WhiteBrush->GetTint(InWidgetStyle) * FLinearColor(1.0f, 1.0f, 0.0f, 1.0f)
			);

		LayerId++;
	}

	if (PreviewData.bShowFilterLow)
	{
		TArray<FVector2D> FilterLine;
		FilterLine.Add(GetWidgetPosition(PreviewData.FilterLow, 0, AllottedGeometry));
		FilterLine.Add(GetWidgetPosition(PreviewData.FilterLow, 1, AllottedGeometry));

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			FilterLine,
			DrawEffects,
			WhiteBrush->GetTint(InWidgetStyle) * FLinearColor(1.0f, 0.0f, 0.0f, 1.0f)
			);

		LayerId++;
	}

	if (PreviewData.bShowFilterHigh)
	{
		TArray<FVector2D> FilterLine;
		FilterLine.Add(GetWidgetPosition(PreviewData.FilterHigh, 0, AllottedGeometry));
		FilterLine.Add(GetWidgetPosition(PreviewData.FilterHigh, 1, AllottedGeometry));

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			FilterLine,
			DrawEffects,
			WhiteBrush->GetTint(InWidgetStyle) * FLinearColor(1.0f, 0.0f, 0.0f, 1.0f)
			);

		LayerId++;
	}

	// Draw line graph
	TArray<FVector2D> LinePoints;

	const float DeltaX = (AllottedGeometry.Size.X / (PreviewData.Samples.Num() - 1));
	for (int32 Idx = 0; Idx < PreviewData.Samples.Num(); Idx++)
	{
		const float XPos = Idx * DeltaX;
		const float YPos = (AllottedGeometry.Size.Y - 1) - (PreviewData.Samples[Idx] * AllottedGeometry.Size.Y);

		LinePoints.Add(FVector2D(FMath::TruncToFloat(XPos), FMath::TruncToFloat(YPos)));
	}

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		LinePoints,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * FLinearColor(0.0f, 0.0f, 1.0f, 1.0f)
		);

	LayerId++;

	return LayerId;
}

FVector2D STestFunctionWidget::ComputeDesiredSize( float ) const
{
	return FVector2D(128, 92);
}
