// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
class UEnvQueryTest;

/**
* Debug console widget, designed to be summoned on top of a viewport or window
*/
class STestFunctionWidget : public SLeafWidget
{
public:

	SLATE_BEGIN_ARGS(STestFunctionWidget)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	TWeakObjectPtr<UEnvQueryTest> DrawTestOb;

private:

	FVector2D GetWidgetPosition(float X, float Y, const FGeometry& Geom) const;
};
