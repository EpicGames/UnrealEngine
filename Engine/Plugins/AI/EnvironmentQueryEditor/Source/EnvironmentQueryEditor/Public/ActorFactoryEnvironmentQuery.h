// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryEnvironmentQuery.generated.h"

class AActor;
struct FAssetData;

UCLASS(MinimalAPI, config=Editor, collapsecategories, hidecategories=Object)
class UActorFactoryEnvironmentQuery : public UActorFactory
{
	GENERATED_BODY()

	UActorFactoryEnvironmentQuery();

protected:
	//~ Begin UActorFactory Interface
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	//~ End UActorFactory Interface
};



