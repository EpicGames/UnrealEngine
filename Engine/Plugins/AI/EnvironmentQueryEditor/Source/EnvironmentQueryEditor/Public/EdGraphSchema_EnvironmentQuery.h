// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraphSchema.h"
#include "EdGraphSchema_EnvironmentQuery.generated.h"

struct FGraphContextMenuBuilder;
struct FPinConnectionResponse;

class UEdGraph;

UCLASS(MinimalAPI)
class UEdGraphSchema_EnvironmentQuery : public UAIGraphSchema
{
	GENERATED_UCLASS_BODY()

	//~ Begin EdGraphSchema Interface
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual const FPinConnectionResponse CanMergeNodes(const UEdGraphNode* A, const UEdGraphNode* B) const override;
	//~ End EdGraphSchema Interface

	//~ Begin UAIGraphSchema Interface
	virtual void GetSubNodeClasses(int32 SubNodeFlags, TArray<FGraphNodeClassData>& ClassData, UClass*& GraphNodeClass) const override;
	//~ End UAIGraphSchema Interface
};
