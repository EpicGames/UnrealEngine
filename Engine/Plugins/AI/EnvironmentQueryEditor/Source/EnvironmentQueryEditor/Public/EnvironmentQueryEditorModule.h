// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"

class IEnvironmentQueryEditor;
struct FGraphNodeClassHelper;
class IToolkitHost;

DECLARE_LOG_CATEGORY_EXTERN(LogEnvironmentQueryEditor, Log, All);

class FEnvironmentQueryEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Creates an instance of EQS editor.  Only virtual so that it can be called across the DLL boundary. */
	virtual TSharedRef<IEnvironmentQueryEditor> CreateEnvironmentQueryEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, class UEnvQuery* Query);

	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	/** EQS Editor app identifier string */
	static const FName EnvironmentQueryEditorAppIdentifier;

	TSharedPtr<FGraphNodeClassHelper> GetClassCache() { return ClassCache; }

private:
	TSharedPtr<FGraphNodeClassHelper> ClassCache;

	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
};
