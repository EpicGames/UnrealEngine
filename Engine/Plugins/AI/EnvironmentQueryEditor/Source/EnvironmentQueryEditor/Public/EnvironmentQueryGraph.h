// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AIGraph.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "EnvironmentQueryGraph.generated.h"

class UEnvironmentQueryGraphNode;
class UEnvQueryOption;
class UEnvQueryTest;

UCLASS()
class UEnvironmentQueryGraph : public UAIGraph
{
	GENERATED_UCLASS_BODY()

	virtual void Initialize() override;
	virtual void OnLoaded() override;
	virtual void UpdateVersion() override;
	virtual void MarkVersion() override;
	virtual void UpdateAsset(int32 UpdateFlags = 0) override;

	void UpdateDeprecatedGeneratorClasses();
	void SpawnMissingNodes();
	void CalculateAllWeights();
	void CreateEnvQueryFromGraph(class UEnvironmentQueryGraphNode* RootEdNode);

	void ResetProfilerStats();
#if USE_EQS_DEBUGGER
	void StoreProfilerStats(const FEQSDebugger::FStatsInfo& Stats);
#endif

protected:

	void UpdateVersion_NestedNodes();
	void UpdateVersion_FixupOuters();
	void UpdateVersion_CollectClassData();

	virtual void CollectAllNodeInstances(TSet<UObject*>& NodeInstances) override;
	virtual void OnNodeInstanceRemoved(UObject* NodeInstance) override;
	virtual void OnNodesPasted(const FString& ImportStr) override;

	void SpawnMissingSubNodes(UEnvQueryOption* Option, TSet<UEnvQueryTest*> ExistingTests, UEnvironmentQueryGraphNode* OptionNode);
};
