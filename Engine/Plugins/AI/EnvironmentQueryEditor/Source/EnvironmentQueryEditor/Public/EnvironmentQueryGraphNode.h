// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraphNode.h"
#include "EnvironmentQueryGraphNode.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UEdGraphSchema;

struct FEnvionmentQueryNodeStats
{
	float MaxTime;
	float AvgTime;
	int32 MaxNumProcessedItems;

	FEnvionmentQueryNodeStats() : MaxTime(0.0f), AvgTime(0.0f), MaxNumProcessedItems(0) {}
};

UCLASS()
class UEnvironmentQueryGraphNode : public UAIGraphNode
{
	GENERATED_UCLASS_BODY()

	virtual class UEnvironmentQueryGraph* GetEnvironmentQueryGraph();

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetDescription() const override;

	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const override;
};
