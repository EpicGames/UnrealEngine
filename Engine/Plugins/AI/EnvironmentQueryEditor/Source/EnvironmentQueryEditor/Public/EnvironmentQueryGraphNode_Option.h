// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnvironmentQueryGraphNode.h"
#include "EnvironmentQueryGraphNode_Option.generated.h"

class UEdGraph;

UCLASS()
class UEnvironmentQueryGraphNode_Option : public UEnvironmentQueryGraphNode
{
	GENERATED_UCLASS_BODY()

	uint32 bStatShowOverlay : 1;
	TArray<FEnvionmentQueryNodeStats> StatsPerGenerator;
	float StatAvgPickRate;

	virtual void AllocateDefaultPins() override;
	virtual void PostPlacedNewNode() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetDescription() const override;

	virtual void PrepareForCopying() override;
	virtual void UpdateNodeClassData() override;

	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	void CreateAddTestSubMenu(class UToolMenu* Menu, UEdGraph* Graph) const;

	void CalculateWeights();
	void UpdateNodeData();

protected:

	virtual void ResetNodeOwner() override;
};
