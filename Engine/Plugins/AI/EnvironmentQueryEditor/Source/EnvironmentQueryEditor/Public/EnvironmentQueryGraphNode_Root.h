// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EnvironmentQueryGraphNode.h"
#include "EnvironmentQueryGraphNode_Root.generated.h"

UCLASS()
class UEnvironmentQueryGraphNode_Root : public UEnvironmentQueryGraphNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Debug)
	TArray<FString> DebugMessages;

	UPROPERTY()
	bool bHasDebugError;

	void LogDebugMessage(const FString& Message);
	void LogDebugError(const FString& Message);

	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool HasErrors() const override { return false; }
};
