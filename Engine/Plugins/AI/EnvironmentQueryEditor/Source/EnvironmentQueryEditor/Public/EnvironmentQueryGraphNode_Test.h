// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnvironmentQueryGraphNode.h"
#include "EnvironmentQueryGraphNode_Test.generated.h"

UCLASS()
class UEnvironmentQueryGraphNode_Test : public UEnvironmentQueryGraphNode
{
	GENERATED_UCLASS_BODY()

	/** weight percent for display */
	UPROPERTY()
	float TestWeightPct;

	/** weight is passed as named param */
	UPROPERTY()
	uint32 bHasNamedWeight : 1;

	/** toggles test */
	UPROPERTY()
	uint32 bTestEnabled : 1;

	uint32 bStatShowOverlay : 1;
	FEnvionmentQueryNodeStats Stats;

	virtual void InitializeInstance() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetDescription() const override;

	void SetDisplayedWeight(float Pct, bool bNamed);
};
